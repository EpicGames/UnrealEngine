// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderTraceAnalyzer.h"
#include "IRenderTraceProvider.h"
#include "RenderTraceModule.h"
#include "TraceServices/Model/Threads.h"
#include "InsightsCore/Common/TimeUtils.h"
#include "Common/ProviderLock.h"

namespace UE
{
namespace RenderTraceInsights
{

enum ECommandListOpType
{
	CommandListOp_Open = 0,
	CommandListOp_Close = 1
};

enum EFenceOpType
{
	FenceOp_Wait = 0,
	FenceOp_Signal = 1
};

FRenderTraceAnalyzer::FRenderTraceAnalyzer(TraceServices::IAnalysisSession& InSession, IEditableRenderTraceProvider& InProvider)
	: Session(InSession)
	, Provider(InProvider)
{}

void FRenderTraceAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	UE::Trace::IAnalyzer::FInterfaceBuilder& Builder = Context.InterfaceBuilder;

#define ROUTE_EVENT(EventName) Builder.RouteEvent(Packet_##EventName, "RenderTrace", #EventName "Message");

	ROUTE_EVENT(CommandListCreated);
	ROUTE_EVENT(CommandListDetach);
	ROUTE_EVENT(CommandListFinishRecording);
	ROUTE_EVENT(CommandListDestroyed);
	ROUTE_EVENT(RDGPassNameSpec);
	ROUTE_EVENT(BeginExecuteRDGPass);
	ROUTE_EVENT(EndExecuteRDGPass);
	ROUTE_EVENT(BeginExecuteDispatchPass);
	ROUTE_EVENT(AddDispatchPassCommandList);
	ROUTE_EVENT(EndExecuteDispatchPass);
	ROUTE_EVENT(SubmitCommandLists);
	ROUTE_EVENT(SplitTranslateJob);
	ROUTE_EVENT(DispatchCommandList);
	ROUTE_EVENT(BeginTranslateCommandList);
	ROUTE_EVENT(ActivatePipelines);
	ROUTE_EVENT(SetTranslateContext);
	ROUTE_EVENT(EndTranslateCommandList);
	ROUTE_EVENT(FinalizeTranslateJob);
	ROUTE_EVENT(SubmitTranslateJobs);
	ROUTE_EVENT(AssociateContext);
	ROUTE_EVENT(PayloadCreated);
	ROUTE_EVENT(AddPayload);
	ROUTE_EVENT(CommandListOp);
	ROUTE_EVENT(SyncPointOp);
	ROUTE_EVENT(ManualFenceOp);
	ROUTE_EVENT(ProcessSubmissionQueueEnter);
	ROUTE_EVENT(SubmitYieldOnUnresolvedSyncPoint);
	ROUTE_EVENT(SubmitYieldOnUnresolvedManualFence);
	ROUTE_EVENT(SubmitWaitQueueFence);
	ROUTE_EVENT(SubmitWaitManualFence);
	ROUTE_EVENT(SubmitPlatformCommandLists);
	ROUTE_EVENT(SubmitSignalManualFence);
	ROUTE_EVENT(SubmitSignalQueueFence);
	ROUTE_EVENT(SubmitResolveSyncPoint);
	ROUTE_EVENT(ProcessSubmissionQueueExit);
	ROUTE_EVENT(ProcessInterruptQueueEnter);
	ROUTE_EVENT(InterruptQueueFenceSignaled);
	ROUTE_EVENT(ProcessInterruptQueueExit);

#undef ROUTE_EVENT
}

void FRenderTraceAnalyzer::BeginCmdListExecute(uint64 CmdListAppID, uint32 PermanentPassID, double Timestamp)
{
	FRDGPassInstance& Pass = Provider.EditRDGPass(PermanentPassID);

	const uint32* PermanentCmdListID = LiveCommandLists.Find(CmdListAppID);
	if (!PermanentCmdListID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got unknown command list %llx for begin execute RDG pass for '%ls' (permanent ID %x), ignoring.",
			CmdListAppID, Pass.Name, PermanentPassID);
		return;
	}

	FCommandListInstance& CmdList = Provider.EditCommandList(*PermanentCmdListID);
	if (!CmdList.RecordingEvents.IsEmpty() && CmdList.RecordingEvents.Last().End == 0.0)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got begin execute RDG pass for '%ls' (permanent ID %x) on command list %llx which has an open recording event (ID %x). Ignoring.",
			Pass.Name, PermanentPassID, CmdList.AppID, CmdList.RecordingEvents.Last().ItemID);
		return;
	}

	Pass.CommandListIDs.Add(*PermanentCmdListID);
	CmdList.RecordingEvents.Emplace(PermanentPassID, Timestamp);
}

void FRenderTraceAnalyzer::EndRDGPassExecute(uint32 PermanentPassID, double Timestamp)
{
	FRDGPassInstance& Pass = Provider.EditRDGPass(PermanentPassID);
	Pass.EndTime = Timestamp;

	for (uint32 PermanentCmdListID : Pass.CommandListIDs)
	{
		FCommandListInstance& CmdList = Provider.EditCommandList(PermanentCmdListID);
		check(!CmdList.RecordingEvents.IsEmpty() && CmdList.RecordingEvents.Last().ItemID == PermanentPassID);

		// Sometimes FinishRecording arrives before a pass ends, e.g. for dispatch passes. When that happens it closes the recording
		// interval, so we should only set the end timestamp if it's not already set.
		if (CmdList.RecordingEvents.Last().End == 0.0)
		{
			CmdList.RecordingEvents.Last().End = Timestamp;
		}
	}

	IRenderTraceProvider::TEventTimeline& Timeline = Provider.EditRDGThreadTimeline(Pass.ExecThreadID);
	Timeline.AppendEndEvent(Timestamp);
}

void FRenderTraceAnalyzer::AddCommandListToTimeline(uint32 PermanentCmdListID)
{
	FCommandListInstance& CmdList = Provider.EditCommandList(PermanentCmdListID);

	// A positive timeline index means the command list was already added, e.g. in FinishRecording and now we're called from
	// its destroy event. A negative value different from -1 means we've already processed this command list and decided not
	// to add it to a timeline.
	if (CmdList.TimelineIndex != -1)
	{
		return;
	}

	int32 FirstEligibleTimeline;
	double StartTime, EndTime;
	switch (CmdList.Type)
	{
	case ECommandListType::Immediate:
	{
		// Try to place immediate command lists on the first timeline. Other command list types will set FirstEligibleTimeline to 1
		// so they're never placed on the first lane.
		FirstEligibleTimeline = 0;
		StartTime = CmdList.CreateTime;
		EndTime = FMath::Max(CmdList.FinishRecordingTime, CmdList.DestroyTime);
		break;
	}

	case ECommandListType::Detached:
	{
		// Skip detached command lists if the parent won't be shown.
		if (CmdList.SourceCmdListID == INVALID_EVENT_ID)
		{
			CmdList.TimelineIndex = -2;
			return;
		}

		const FCommandListInstance& SourceCmdList = Provider.EditCommandList(CmdList.SourceCmdListID);
		if (SourceCmdList.RecordingEvents.IsEmpty())
		{
			CmdList.TimelineIndex = -2;
			return;
		}

		FirstEligibleTimeline = 1;
		StartTime = CmdList.CreateTime;
		EndTime = FMath::Max(CmdList.FinishRecordingTime, StartTime);
		break;
	}

	case ECommandListType::Regular:
	{
		// Skip regular command lists with no recording events.
		if (CmdList.RecordingEvents.IsEmpty())
		{
			CmdList.TimelineIndex = -2;
			return;
		}

		FirstEligibleTimeline = 1;

		// Nothing interesting happens before the first recording event, so start the command list event then.
		StartTime = CmdList.RecordingEvents[0].Start;
		EndTime = FMath::Max(CmdList.FinishRecordingTime, CmdList.RecordingEvents.Last().End);
		break;
	}

	default:
		checkNoEntry();
		return;
	}

	CmdList.TimelineStart = StartTime;

	if (StartTime == EndTime)
	{
		// The tiny command lists made for closing out parallel dispatch passes finish recording as soon as they're created,
		// so nudge their end time a little bit in order to make them show up in the timeline.
		EndTime += UE::Insights::FTimeValue::Microsecond;
	}

	int32 BestTimelineIdx = -1;
	IRenderTraceProvider::TEventTimeline* BestTimeline = nullptr;
	double BestDist = std::numeric_limits<double>::infinity();
	const int32 NumTimelines = Provider.GetNumCommandListTimelines();
	for (int32 TimelineIdx = FirstEligibleTimeline; TimelineIdx < NumTimelines; ++TimelineIdx)
	{
		IRenderTraceProvider::TEventTimeline& Timeline = Provider.EditCommandListTimeline(TimelineIdx);
		double Distance = StartTime - Timeline.GetLastTimestamp();
		if (Distance >= 0 && Distance < BestDist)
		{
			BestDist = Distance;
			BestTimeline = &Timeline;
			BestTimelineIdx = TimelineIdx;
			if (TimelineIdx == 0)
			{
				// If we're able to place it on the first track, this is an immediate command list, so stop searching
				// for better matches.
				break;
			}
		}
	}

	if (BestTimeline == nullptr)
	{
		TPair<uint32, IRenderTraceProvider::TEventTimeline&> NewTimeline = Provider.AddCommandListTimeline();
		BestTimelineIdx = NewTimeline.Key;
		// PVS-Studio doesn't understand how references work, we need to disable the warning about taking the address of a variable which goes out of scope.
		BestTimeline = &NewTimeline.Value; //-V506
	}

	CmdList.TimelineIndex = BestTimelineIdx;
	BestTimeline->AppendBeginEvent(StartTime, FRenderTraceEvent(PermanentCmdListID));

	for (int32 RecordEventIdx = 0; RecordEventIdx < CmdList.RecordingEvents.Num(); ++RecordEventIdx)
	{
		FEventInterval& RecordEvent = CmdList.RecordingEvents[RecordEventIdx];
		BestTimeline->AppendBeginEvent(RecordEvent.Start, FRenderTraceEvent(RecordEvent.ItemID, PermanentCmdListID));

		if (RecordEvent.End == 0.0)
		{
			// Close the event if it's still open. This can happen if FinishRecording arrives before a pass ends, e.g. for parallel dispatch passes.
			if (RecordEventIdx == CmdList.RecordingEvents.Num() - 1)
			{
				RecordEvent.End = EndTime;
			}
			else
			{
				UE_LOGF(LogRenderTrace, Warning, "Command list %llx (permanent ID %x) is being added to the timeline with an open recording event that's not the last in the list (index %d, num %d).",
					CmdList.AppID, PermanentCmdListID, RecordEventIdx, CmdList.RecordingEvents.Num());
			}
		}

		if (RecordEvent.End <= RecordEvent.Start)
		{
			// Some command lists (e.g. pass epilogues) finish recording as soon as they're created, so we nudge the
			// end of the recording event a little bit, but we need to be careful not to overlap the next event.
			const double NextEventStart = (RecordEventIdx < CmdList.RecordingEvents.Num() - 1) ? CmdList.RecordingEvents[RecordEventIdx + 1].Start : EndTime;
			RecordEvent.End = FMath::Min(RecordEvent.Start + UE::Insights::FTimeValue::Microsecond, NextEventStart);
		}

		BestTimeline->AppendEndEvent(RecordEvent.End);
	}

	BestTimeline->AppendEndEvent(EndTime);
}

void FRenderTraceAnalyzer::EndCommandList(uint32 PermanentCmdListID, double Timestamp)
{
	FCommandListInstance& CmdList = Provider.EditCommandList(PermanentCmdListID);
	if (!CmdList.RecordingEvents.IsEmpty() && CmdList.RecordingEvents.Last().End == 0.0)
	{
		// It's possible to receive a destroy event for the immediate command list while a pass is recording to it, e.g. FRDGDispatchPass::Execute() calls
		// QueueAsyncCommandListSubmit(), which can detach the pending commands in the immediate command list. We only log a warning if this happens for
		// other command list types. In both cases we close the pending recording event.
		if (CmdList.Type != ECommandListType::Immediate)
		{
			UE_LOGF(LogRenderTrace, Warning, "Command list %llx (permanent ID %x) ended with open recording event for pass %x, force-closing.",
				CmdList.AppID, PermanentCmdListID, CmdList.RecordingEvents.Last().ItemID);
		}
		CmdList.RecordingEvents.Last().End = Timestamp;
	}
	CmdList.DestroyTime = Timestamp;

	AddCommandListToTimeline(PermanentCmdListID);
}

void FRenderTraceAnalyzer::AddTranslateTasksToTimeline(uint32 ThreadID)
{
	TArray<FLiveTranslateJob*, TInlineAllocator<8>> PendingJobs;
	for (auto& It : LiveTranslateJobs)
	{
		const FRHITranslateTask& PendingTask = Provider.GetRHITranslateTask(It.Value.PermanentID);
		if (!It.Value.bAddedToTimeline && PendingTask.ThreadID == ThreadID)
		{
			PendingJobs.Add(&It.Value);
		}
	}

	if (PendingJobs.IsEmpty())
	{
		return;
	}

	// Sort the pending jobs by start time.
	Algo::Sort(PendingJobs, [this](const FLiveTranslateJob* LiveJob1, const FLiveTranslateJob* LiveJob2) {
		const FRHITranslateTask& Task1 = Provider.GetRHITranslateTask(LiveJob1->PermanentID);
		const FRHITranslateTask& Task2 = Provider.GetRHITranslateTask(LiveJob2->PermanentID);
		return Task1.StartTime < Task2.StartTime;
	});

	IRenderTraceProvider::TEventTimeline& Timeline = Provider.EditRHITranslateThreadTimeline(ThreadID);

	// Add events until we find one that's still pending.
	for (FLiveTranslateJob* LiveJob : PendingJobs)
	{
		const FRHITranslateTask& Task = Provider.GetRHITranslateTask(LiveJob->PermanentID);
		if (Task.EndTime == 0.0)
		{
			break;
		}

		Timeline.AppendBeginEvent(Task.StartTime, FRenderTraceEvent(LiveJob->PermanentID));

		// Add subevents for the command lists which are translated.
		for (const FEventInterval& CmdList : Task.ProcessedItems)
		{
			Timeline.AppendBeginEvent(CmdList.Start, FRenderTraceEvent(CmdList.ItemID, LiveJob->PermanentID));
			Timeline.AppendEndEvent(CmdList.End);
		}

		Timeline.AppendEndEvent(Task.EndTime);

		LiveJob->bAddedToTimeline = true;
	}
}

void FRenderTraceAnalyzer::EndTranslateTask(uint32 PermanentTaskID, double Timestamp)
{
	FRHITranslateTask& TranslateTask = Provider.EditRHITranslateTask(PermanentTaskID);
	check(TranslateTask.Type == ERHITranslateTaskType::Translate);

	if (TranslateTask.EndTime == 0.0)
	{
		TranslateTask.EndTime = Timestamp;
	}

	for (int PipeIdx = 0; PipeIdx < 2; ++PipeIdx)
	{
		FRHITranslateContext& TranslateContext = TranslateTask.Contexts[PipeIdx];
		if (!TranslateContext.ActiveIntervals.IsEmpty() && TranslateContext.ActiveIntervals.Last().End == 0.0)
		{
			TranslateContext.ActiveIntervals.Last().End = TranslateTask.EndTime;
		}

		for (uint32 PayloadID : TranslateContext.PayloadIDs)
		{
			FPlatformPayload& Payload = Provider.EditPlatformPayload(PayloadID);
			if (Payload.EndTime == 0.0)
			{
				Payload.EndTime = TranslateTask.EndTime;
			}

			Payload.StartTime = FMath::Min(Payload.StartTime, TranslateTask.EndTime);
			Payload.EndTime = FMath::Min(Payload.EndTime, TranslateTask.EndTime);
		}
	}

	// We can't add the task to the timeline right away because jobs can be translated on one thread and finalized
	// on another, which means finalize events can arrive in a different order than translate events. LiveTranslateJobs
	// doubles as a queue of pending jobs, and we drain as much as possible from it each time this function is called.
	AddTranslateTasksToTimeline(TranslateTask.ThreadID);
}

FPlatformPayload* FRenderTraceAnalyzer::FindPlatformPayload(uint64 PayloadID, const TCHAR* EventName, uint32 ThreadID)
{
	const uint32* ExistingPermanentID = LivePlatformPayloads.Find(PayloadID);
	if (!ExistingPermanentID)
	{
		if (ThreadID != GameThreadID)
		{
			UE_LOGF(LogRenderTrace, Warning, "Got %ls event for payload ID %llx, but the payload is not in the live payloads list.", EventName, PayloadID);
		}
		return nullptr;
	}

	return &Provider.EditPlatformPayload(*ExistingPermanentID);
}

void FRenderTraceAnalyzer::ProcessCommandListCreated(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 CmdListID = EventData.GetValue<uint64>("CmdListID");
	const bool bImmediate = EventData.GetValue<bool>("bImmediate");

	const uint32* ExistingPermanentID = LiveCommandLists.Find(CmdListID);
	if (ExistingPermanentID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got create command list for ID %llx while it's already open, ignoring.", CmdListID);
		return;
	}

	TPair<uint32, FCommandListInstance&> NewCmdListEntry = Provider.AddCommandList(CmdListID, Timestamp, bImmediate ? ECommandListType::Immediate : ECommandListType::Regular);
	LiveCommandLists.Add(CmdListID, NewCmdListEntry.Key);
}

void FRenderTraceAnalyzer::ProcessCommandListDetach(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 SourceCmdListID = EventData.GetValue<uint64>("SourceCmdListID");
	const uint64 DetachedCmdListID = EventData.GetValue<uint64>("DetachedCmdListID");
	const bool bIsImmediateFlush = EventData.GetValue<bool>("bIsImmediateFlush");

	const uint32* PermanentSourceCmdListIDPtr = LiveCommandLists.Find(SourceCmdListID);
	if (!PermanentSourceCmdListIDPtr)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got detach command list event for unknown command list ID %llx.", SourceCmdListID);
		return;
	}
	const uint32 PermanentSourceCmdListID = *PermanentSourceCmdListIDPtr;

	// The new command list is created using a move constructor which doesn't send CommandListCreated events for now. Add it to the live map
	// if it's not there already.
	uint32 PermanentDetachedCmdListID;
	const uint32* ExistingDetachedPermanentID = LiveCommandLists.Find(DetachedCmdListID);
	if (!ExistingDetachedPermanentID)
	{
		TPair<uint32, FCommandListInstance&> NewCmdListEntry = Provider.AddCommandList(DetachedCmdListID, Timestamp, ECommandListType::Detached);
		PermanentDetachedCmdListID = NewCmdListEntry.Key;
		LiveCommandLists.Add(DetachedCmdListID, PermanentDetachedCmdListID);
	}
	else
	{
		PermanentDetachedCmdListID = *ExistingDetachedPermanentID;
	}

	FCommandListInstance& SourceCmdList = Provider.EditCommandList(PermanentSourceCmdListID);
	if (SourceCmdList.DetachedCmdListID != INVALID_EVENT_ID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got detach command list event for source command list %llx to new command list %llx, but the source has already been detached to permanent ID %x.",
			SourceCmdListID, DetachedCmdListID, SourceCmdList.DetachedCmdListID);
	}
	SourceCmdList.DetachedCmdListID = PermanentDetachedCmdListID;
	SourceCmdList.DetachTime = Timestamp;

	FCommandListInstance& DetachedCmdList = Provider.EditCommandList(PermanentDetachedCmdListID);
	DetachedCmdList.Type = ECommandListType::Detached;
	DetachedCmdList.SourceCmdListID = PermanentSourceCmdListID;
}

void FRenderTraceAnalyzer::ProcessCommandListFinishRecording(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 CmdListID = EventData.GetValue<uint64>("CmdListID");
	const uint8 Flags = EventData.GetValue<uint8>("Flags");

	const uint32* PermanentCmdListID = LiveCommandLists.Find(CmdListID);
	if (!PermanentCmdListID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got finish recording event for unknown command list ID %llx.", CmdListID);
		return;
	}

	FCommandListInstance& CmdList = Provider.EditCommandList(*PermanentCmdListID);
	CmdList.FinishRecordingTime = Timestamp;
	CmdList.RecordingFlags = Flags;

	AddCommandListToTimeline(*PermanentCmdListID);
}

void FRenderTraceAnalyzer::ProcessCommandListDestroyed(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 CmdListID = EventData.GetValue<uint64>("CmdListID");
	const uint32* PermanentCmdListID = LiveCommandLists.Find(CmdListID);
	if (!PermanentCmdListID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got destroy event for unknown command list ID %llx.", CmdListID);
		return;
	}

	EndCommandList(*PermanentCmdListID, Timestamp);
	LiveCommandLists.Remove(CmdListID);
}

void FRenderTraceAnalyzer::ProcessRDGPassNameSpec(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint32 NameID = EventData.GetValue<uint32>("NameID");

	FString Name;
	EventData.GetString("Name", Name);

	PassNameLookup.Add(NameID, Session.StoreString(Name));
}

void FRenderTraceAnalyzer::ProcessBeginExecuteRDGPass(const FEventData& EventData, uint32 ThreadID, double Timestamp, bool bIsDispatch)
{
	const uint64 PassID = EventData.GetValue<uint64>("PassID");
	const uint32 NameID = EventData.GetValue<uint32>("NameID");
	
	const TCHAR* Name = TEXT("UNKNOWN");

	// A name ID of 0 means that pass names were disabled in the build which produced the trace. We don't need to warn about that.
	if(NameID > 0)
	{
		const TCHAR* const* FoundName = PassNameLookup.Find(NameID);
		if(FoundName)
		{
			Name = *FoundName;
		}
		else
		{
			UE_LOGF(LogRenderTrace, Warning, "Got unknown name ID %u for RDG pass %llx.", NameID, PassID);
		}
	}

	const uint32* ExistingPermanentID = LiveRDGPasses.Find(PassID);
	if (ExistingPermanentID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got execute RDG pass for '%ls' (app ID %llx) while it's already open (permanent ID %x), ignoring.",
			Name, PassID, *ExistingPermanentID);
		return;
	}

	TPair<uint32, FRDGPassInstance&> NewPassEntry = Provider.AddRDGPass(Name, ThreadID, Timestamp, bIsDispatch ? ERDGPassType::Dispatch : ERDGPassType::Regular);
	LiveRDGPasses.Add(PassID, NewPassEntry.Key);

	if (!bIsDispatch)
	{
		const uint64 CmdListID = EventData.GetValue<uint64>("CmdListID");
		BeginCmdListExecute(CmdListID, NewPassEntry.Key, Timestamp);

		NewPassEntry.Value.Flags = EventData.GetValue<uint16>("PassFlags");
		NewPassEntry.Value.TaskMode = static_cast<ERDGPassTaskMode>(EventData.GetValue<uint8>("TaskMode"));
	}

	IRenderTraceProvider::TEventTimeline& Timeline = Provider.EditRDGThreadTimeline(ThreadID);
	Timeline.AppendBeginEvent(Timestamp, FRenderTraceEvent(NewPassEntry.Key));
}

void FRenderTraceAnalyzer::ProcessEndExecuteRDGPass(const FEventData& EventData, uint32 ThreadID, double Timestamp, bool bIsDispatch)
{
	const uint64 PassID = EventData.GetValue<uint64>("PassID");
	const uint32* PermanentPassID = LiveRDGPasses.Find(PassID);
	if (!PermanentPassID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got end execute RDG pass for unknown pass ID %llx.", PassID);
		return;
	}

	if (bIsDispatch)
	{
		const uint64 EpilogueCmdListID = EventData.GetValue<uint64>("EpilogueCmdListID");
		if (EpilogueCmdListID > 0)
		{
			BeginCmdListExecute(EpilogueCmdListID, *PermanentPassID, Timestamp);
		}
	}

	EndRDGPassExecute(*PermanentPassID, Timestamp);
	LiveRDGPasses.Remove(PassID);
}

void FRenderTraceAnalyzer::ProcessAddDispatchPassCommandList(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 PassID = EventData.GetValue<uint64>("PassID");
	const uint64 CmdListID = EventData.GetValue<uint64>("CmdListID");

	const uint32* PermanentPassID = LiveRDGPasses.Find(PassID);
	if (!PermanentPassID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got add RDG pass command list for unknown pass ID %llx.", PassID);
		return;
	}

	FRDGPassInstance& Pass = Provider.EditRDGPass(*PermanentPassID);
	BeginCmdListExecute(CmdListID, *PermanentPassID, Timestamp);
}

void FRenderTraceAnalyzer::ProcessSubmitCommandLists(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	TConstArrayView<uint64> CmdListIDs = EventData.GetArrayView<uint64>("CmdListIDs");
	const uint32 RTSubmissionID = EventData.GetValue<uint32>("RTSubmissionID");
	const uint16 Flags = EventData.GetValue<uint16>("Flags");

	// We assign an arbitrary duration of 10 us to these events so that they can be seen on the timeline. If a second submission
	// event arrives less than 10 us after we added the previous one, we need to nudge it slightly to fix overlaps.
	IRenderTraceProvider::TEventTimeline& Timeline = Provider.EditRenderThreadSubmissionTimeline();
	const double EventStartTime = FMath::Max(Timestamp, Timeline.GetLastTimestamp());

	// Add a fake RDG pass to represent the submit.
	const TCHAR* PassName = (RTSubmissionID != 0) ? TEXT("SubmitToGPU") : TEXT("Queue Submit");
	TPair<uint32, FRDGPassInstance&> SubmitPassEntry = Provider.AddRDGPass(PassName, ThreadID, EventStartTime, ERDGPassType::Submit);
	FRDGPassInstance& Pass = SubmitPassEntry.Value;
	Pass.Flags = Flags;
	Pass.EndTime = Pass.StartTime + 10 * UE::Insights::FTimeValue::Microsecond;

	if (RTSubmissionID != 0)
	{
		// Remember which pass ID corresponds to the app-provided submission ID.
		PendingRTSubmitIDs.Add(RTSubmissionID, SubmitPassEntry.Key);
	}

	Pass.CommandListIDs.Reserve(CmdListIDs.Num());
	for (uint64 CmdListID : CmdListIDs)
	{
		const uint32* PermanentCmdListID = LiveCommandLists.Find(CmdListID);
		if (!PermanentCmdListID)
		{
			UE_LOGF(LogRenderTrace, Warning, "Got submit event for unknown command list ID %llx.", CmdListID);
			continue;
		}

		Pass.CommandListIDs.Add(*PermanentCmdListID);
		FCommandListInstance& CmdList = Provider.EditCommandList(*PermanentCmdListID);
		CmdList.SubmitTime = Timestamp;
		CmdList.SubmitPassID = SubmitPassEntry.Key;
	}

	Timeline.AppendBeginEvent(Pass.StartTime, FRenderTraceEvent(SubmitPassEntry.Key));
	Timeline.AppendEndEvent(Pass.EndTime);
}

void FRenderTraceAnalyzer::ProcessSplitTranslateJob(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 TranslateJobID = EventData.GetValue<uint64>("TranslateJobID");
	const uint64 CmdListID = EventData.GetValue<uint64>("CmdListID");
	const uint8 Flags = EventData.GetValue<uint8>("Flags");

	const uint32* PermanentCmdListID = LiveCommandLists.Find(CmdListID);
	if (!PermanentCmdListID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got split event with unknown command list ID %llx.", CmdListID);
		return;
	}

	// We store this information on the command list because the translate task object might not exist yet.
	// A translate tasks is only created when it starts to translate its first command list, which can be after
	// the event which caused the previous one to be split.
	FCommandListInstance& CmdList = Provider.EditCommandList(*PermanentCmdListID);
	CmdList.TranslatePrevJobAppID = TranslateJobID;
	CmdList.TranslateSplitFlags = Flags;
}

void FRenderTraceAnalyzer::ProcessDispatchCommandList(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 CmdListID = EventData.GetValue<uint64>("CmdListID");

	const uint32* PermanentCmdListID = LiveCommandLists.Find(CmdListID);
	if (!PermanentCmdListID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got dispatch event for unknown command list ID %llx.", CmdListID);
		return;
	}

	FCommandListInstance& CmdList = Provider.EditCommandList(*PermanentCmdListID);
	CmdList.DispatchTime = Timestamp;
}

void FRenderTraceAnalyzer::ProcessBeginTranslateCommandList(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 TranslateJobID = EventData.GetValue<uint64>("TranslateJobID");
	const uint64 CmdListID = EventData.GetValue<uint64>("CmdListID");

	const uint32* PermanentCmdListID = LiveCommandLists.Find(CmdListID);
	if (!PermanentCmdListID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got begin translate event for unknown command list ID %llx.", CmdListID);
		return;
	}

	FCommandListInstance& CmdList = Provider.EditCommandList(*PermanentCmdListID);
	if (CmdList.TranslateTaskID != INVALID_EVENT_ID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got begin translate event for command list ID %llx which is already associated with task %x.", CmdListID, CmdList.TranslateTaskID);
		return;
	}

	uint32 PermanentTranslateJobID = INVALID_EVENT_ID, ForceSplitFromJobID = INVALID_EVENT_ID;

	const FLiveTranslateJob* LiveTranslateJob = LiveTranslateJobs.Find(TranslateJobID);
	if (LiveTranslateJob)
	{
		PermanentTranslateJobID = LiveTranslateJob->PermanentID;
		FRHITranslateTask& TranslateJob = Provider.EditRHITranslateTask(PermanentTranslateJobID);
		check(TranslateJob.Type == ERHITranslateTaskType::Translate);
		if (TranslateJob.ThreadID != ThreadID)
		{
			// The job just jumped to another thread. This happens if a command list finishes translating before the next one is dispatched,
			// so that the job spawns another taskgraph task. We will close the current job and start a new one.
			EndTranslateTask(PermanentTranslateJobID, Timestamp);
			ForceSplitFromJobID = PermanentTranslateJobID;
			PermanentTranslateJobID = INVALID_EVENT_ID;
		}
	}

	if (PermanentTranslateJobID == INVALID_EVENT_ID)
	{
		TPair<uint32, FRHITranslateTask&> NewTaskEntry = Provider.AddRHITranslateTask(TranslateJobID, ERHITranslateTaskType::Translate, ThreadID, Timestamp);
		PermanentTranslateJobID = NewTaskEntry.Key;
		LiveTranslateJobs.Emplace(TranslateJobID, PermanentTranslateJobID);
	}

	CmdList.TranslateTaskID = PermanentTranslateJobID;

	FRHITranslateTask& TranslateJob = Provider.EditRHITranslateTask(PermanentTranslateJobID);
	TranslateJob.ProcessedItems.Emplace(*PermanentCmdListID, Timestamp);

	if (ForceSplitFromJobID != INVALID_EVENT_ID)
	{
		// We've split the job because it jumped threads, so we need to move the contexts from the old job to the new one.
		// Note that this doesn't move any payloads, so if the new command list is executed on the active payload we won't
		// be able to show this on the timeline.
		const FRHITranslateTask& OldJob = Provider.EditRHITranslateTask(ForceSplitFromJobID);
		for (int PipeIdx = 0; PipeIdx < 2; ++PipeIdx)
		{
			const uint64 RHIContextID = OldJob.Contexts[PipeIdx].RHIContextID;
			if (RHIContextID != 0)
			{
				FRHITranslateContext& Context = TranslateJob.Contexts[PipeIdx];
				Context.RHIContextID = RHIContextID;
				Context.ActiveIntervals.Emplace(Timestamp);
				LiveRHIContexts.Add(RHIContextID, PermanentTranslateJobID);
			}
		}

		// Record the split reason on the command list so we can link up the tasks.
		CmdList.TranslatePrevJobAppID = OldJob.AppID;
		CmdList.TranslateSplitFlags = ERHITranslateJobSplitFlag_JumpThreads;
	}
}

FRHITranslateTask* FRenderTraceAnalyzer::GetTranslateJobForCmdList(uint64 CmdListID, const TCHAR* EventName, uint32 ThreadID, uint32* OutTranslateJobID)
{
	const uint32* PermanentCmdListID = LiveCommandLists.Find(CmdListID);
	if (!PermanentCmdListID)
	{
		// The engine calls SwitchPipeline() on the immediate command list from the game thread at startup, to make sure it has an active context.
		// These calls will not be part of a translate job, but it's fine to ignore them. Only log a warning when this happens later.
		if (ThreadID != GameThreadID)
		{
			UE_LOGF(LogRenderTrace, Warning, "Got %ls event for unknown command list ID %llx.", EventName, CmdListID);
		}
		return nullptr;
	}

	FCommandListInstance& CmdList = Provider.EditCommandList(*PermanentCmdListID);
	if (CmdList.TranslateTaskID == INVALID_EVENT_ID)
	{
		if (ThreadID != GameThreadID)
		{
			UE_LOGF(LogRenderTrace, Warning, "Got %ls event for command list ID %llx which doesn't have a translate job.", EventName, CmdListID);
		}
		return nullptr;
	}

	FRHITranslateTask& TranslateJob = Provider.EditRHITranslateTask(CmdList.TranslateTaskID);
	check(TranslateJob.Type == ERHITranslateTaskType::Translate);

	if (OutTranslateJobID)
	{
		*OutTranslateJobID = CmdList.TranslateTaskID;
	}

	return &TranslateJob;
}

void FRenderTraceAnalyzer::ProcessActivatePipelines(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 CmdListID = EventData.GetValue<uint64>("CmdListID");
	const uint8 Pipelines = EventData.GetValue<uint8>("Pipelines");

	FRHITranslateTask* TranslateJob = GetTranslateJobForCmdList(CmdListID, TEXT("activate pipelines"), ThreadID, nullptr);
	if (!TranslateJob)
	{
		return;
	}

	for (int PipelineIdx = 0; PipelineIdx < 2; ++PipelineIdx)
	{
		const bool bActive = (Pipelines & (1 << PipelineIdx)) != 0;
		FRHITranslateContext& TranslateContext = TranslateJob->Contexts[PipelineIdx];
		if (bActive)
		{
			// Ignore if the context is already active.
			if (TranslateContext.ActiveIntervals.IsEmpty() || TranslateContext.ActiveIntervals.Last().End != 0.0)
			{
				TranslateContext.ActiveIntervals.Emplace(Timestamp);
			}
		}
		else
		{
			// Ignore if the context isn't active.
			if (!TranslateContext.ActiveIntervals.IsEmpty() && TranslateContext.ActiveIntervals.Last().End == 0.0)
			{
				TranslateContext.ActiveIntervals.Last().End = Timestamp;
			}
		}
	}
}

void FRenderTraceAnalyzer::ProcessSetTranslateContext(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 CmdListID = EventData.GetValue<uint64>("CmdListID");
	const uint64 ContextID = EventData.GetValue<uint64>("ContextID");
	const uint8 Pipeline = EventData.GetValue<uint8>("Pipeline");

	uint8 PipelineIdx;
	switch (Pipeline)
	{
	case 1 << 0: PipelineIdx = 0; break;
	case 1 << 1: PipelineIdx = 1; break;
	default:
		UE_LOGF(LogRenderTrace, Warning, "Got set translate context event for command list ID %llx with invalid pipeline mask 0x%x.", CmdListID, Pipeline);
		return;
	}

	uint32 TranslateJobID;
	FRHITranslateTask* TranslateJob = GetTranslateJobForCmdList(CmdListID, TEXT("set translate context"), ThreadID, &TranslateJobID);
	if (!TranslateJob)
	{
		return;
	}

	FRHITranslateContext& TranslateContext = TranslateJob->Contexts[PipelineIdx];
	if (TranslateContext.RHIContextID != 0)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got set translate context event for command list ID %llx with context ID %llx on pipe %u, but the job is already using context %llx on that pipe. Ignoring.",
			CmdListID, ContextID, PipelineIdx, TranslateContext.RHIContextID);
		return;
	}

	// RHIs usually have a pool of contexts, and objects are returned to the pool in their RHIFinalizeContexts() implementation,
	// which runs before the FinalizeTranslateJob event is sent. Due to threading, another translate job can pick up the context
	// before the finalize event can remove it from LiveRHIContexts. This is why we can't check if the context is already in use;
	// instead, we just steal it.
	// Note that we can't send FinalizeTranslateJob before calling RHIFinalizeContexts(), because that function will do more things
	// with the context (such as signal batched sync points) and we still need to find the context -> job association when that
	// happens. The association will still be valid when it does that, because these extra actions happen before returning the context
	// to the pool, so another job can't steal the context while we still need the old association.
	// Example flow with D3D12:
	//		FRHICommandListExecutor::FTranslateState::Translate_CloseChain()
	//			|-> GDynamicRHI->RHICloseTranslateChain()
	//			|	|-> FD3D12DynamicRHI::RHIFinalizeContexts()
	//			|		|-> FD3D12ContextCommon::Finalize()
	//			|		|	|-> Payload->SyncPointsToWait.Append(BatchedSyncPoints.ToWait) - this needs to find the current context -> translate job mapping
	//			|		|-> CmdContext->GetParentDevice()->ReleaseContext() - this releases the context, so another thread can pick it up for its translate job
	// ... another thread sends a SetTranslateContext packet and steals the context for a different job ...
	//			|-> RenderTracing::FinalizeTranslateJob() - we should only manipulate the context if it's still bound to the job
	TranslateContext.RHIContextID = ContextID;
	LiveRHIContexts.Add(ContextID, TranslateJobID);
}

void FRenderTraceAnalyzer::ProcessEndTranslateCommandList(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 CmdListID = EventData.GetValue<uint64>("CmdListID");

	const uint32* PermanentCmdListID = LiveCommandLists.Find(CmdListID);
	if (!PermanentCmdListID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got end translate event for unknown command list ID %llx.", CmdListID);
		return;
	}

	FCommandListInstance& CmdList = Provider.EditCommandList(*PermanentCmdListID);
	if (CmdList.TranslateTaskID == INVALID_EVENT_ID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got end translate event for command list ID %llx which doesn't have a translate task.", CmdListID);
		return;
	}

	FRHITranslateTask& TranslateJob = Provider.EditRHITranslateTask(CmdList.TranslateTaskID);
	check(TranslateJob.Type == ERHITranslateTaskType::Translate);

	if (TranslateJob.ThreadID != ThreadID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got end translate event for command list ID %llx on task %llx, but the current thread %u doesn't match the job's thread %u.",
			CmdListID, TranslateJob.AppID, ThreadID, TranslateJob.ThreadID);
	}

	if (TranslateJob.ProcessedItems.IsEmpty())
	{
		UE_LOGF(LogRenderTrace, Warning, "Got end translate event for command list ID %llx on task %llx, but there's no open translation event.", CmdListID, TranslateJob.AppID);
		return;
	}

	FEventInterval& CmdListEvent = TranslateJob.ProcessedItems.Last();
	if (CmdListEvent.ItemID != *PermanentCmdListID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got end translate event for command list ID %llx (permanent ID %x) on task %llx, but the currently open translation event is for command list permanent ID %x.",
			CmdListID, *PermanentCmdListID, TranslateJob.AppID, CmdListEvent.ItemID);
		return;
	}

	// End the translation event and move the end of the translate job.
	CmdListEvent.End = Timestamp;
	TranslateJob.EndTime = Timestamp;
}

void FRenderTraceAnalyzer::ProcessFinalizeTranslateJob(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 TranslateJobID = EventData.GetValue<uint64>("TranslateJobID");
	const FLiveTranslateJob* LiveTranslateJob = LiveTranslateJobs.Find(TranslateJobID);
	const uint8 Flags = EventData.GetValue<uint8>("Flags");
	if (!LiveTranslateJob)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got finalize event for unknown translate job ID %llx.", TranslateJobID);
		return;
	}

	const uint32 PermanentTranslateJobID = LiveTranslateJob->PermanentID;
	FRHITranslateTask& TranslateJob = Provider.EditRHITranslateTask(PermanentTranslateJobID);
	check(TranslateJob.Type == ERHITranslateTaskType::Translate);
	TranslateJob.JobFlags = Flags;

	EndTranslateTask(PermanentTranslateJobID, Timestamp);

	for (int PipeIdx = 0; PipeIdx < 2; ++PipeIdx)
	{
		FRHITranslateContext& TranslateContext = TranslateJob.Contexts[PipeIdx];
		if (TranslateContext.RHIContextID == 0)
		{
			continue;
		}

		// The context might have been stolen by another translate job before we got the FinalizeTranslateJob
		// notification (see the comment in ProcessSetTranslateContexts() for the full explanation). Only
		// remove it from the live contexts map if we still own it.
		const uint32* ExistingRHIContextMapping = LiveRHIContexts.Find(TranslateContext.RHIContextID);
		if (ExistingRHIContextMapping && *ExistingRHIContextMapping == PermanentTranslateJobID)
		{
			LiveRHIContexts.Remove(TranslateContext.RHIContextID);
		}
	}
}

void FRenderTraceAnalyzer::ProcessSubmitTranslateJobs(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	TConstArrayView<uint64> TranslateJobIDs = EventData.GetArrayView<uint64>("TranslateJobIDs");
	const uint64 SubmissionID = EventData.GetValue<uint64>("SubmissionID");

	// Submit "tasks" are just events, so we'll use an arbitrary duration, but we need to make sure we don't
	// overlap the previous submit event on the submission timeline.
	IRenderTraceProvider::TEventTimeline& Timeline = Provider.EditRHISubmissionTimeline();
	const double EventStartTime = FMath::Max(Timestamp, Timeline.GetLastTimestamp());

	TPair<uint32, FRHITranslateTask&> NewTaskEntry = Provider.AddRHITranslateTask(0, ERHITranslateTaskType::Submit, ThreadID, EventStartTime);
	FRHITranslateTask& SubmitTask = NewTaskEntry.Value;

	// Submit tasks don't have an app-provided ID, so we'll use the internal ID.
	SubmitTask.AppID = NewTaskEntry.Key;
	SubmitTask.EndTime = SubmitTask.StartTime + 10 * UE::Insights::FTimeValue::Microsecond;

	Timeline.AppendBeginEvent(SubmitTask.StartTime, FRenderTraceEvent(NewTaskEntry.Key));

	// If the bottom bit is set, this submit event was instigated by the render thread via ERHISubmitFlags::SubmitToGPU, and the RT
	// submission ID is in the top 32 bits. Otherwise it was instigated by the completion of a translate chain, and the value contains
	// the app ID of the last job in the chain.
	if (SubmissionID & 0x01)
	{
		// Link the render-thread submit event to the submit task.
		const uint32 RTSubmissionID = (uint32)(SubmissionID >> 32);
		const uint32* SubmitPassID = PendingRTSubmitIDs.Find(RTSubmissionID);
		if (SubmitPassID)
		{
			FRDGPassInstance& SubmitPass = Provider.EditRDGPass(*SubmitPassID);
			check(SubmitPass.Type == ERDGPassType::Submit);
			SubmitPass.SubmitTaskID = NewTaskEntry.Key;

			SubmitTask.bEagerSubmission = false;
			SubmitTask.SubmitTaskID = *SubmitPassID;

			PendingRTSubmitIDs.Remove(RTSubmissionID);
		}
		else
		{
			UE_LOGF(LogRenderTrace, Warning, "Got submit event with unknown render thread pass ID %x, not linking.", RTSubmissionID);
		}
	}
	else
	{
		const FLiveTranslateJob* LiveTranslateJob = LiveTranslateJobs.Find(SubmissionID);
		if (LiveTranslateJob)
		{
			FRHITranslateTask& InstigatorTask = Provider.EditRHITranslateTask(LiveTranslateJob->PermanentID);
			check(InstigatorTask.Type == ERHITranslateTaskType::Translate);
			InstigatorTask.SubmitTaskID = NewTaskEntry.Key;

			SubmitTask.bEagerSubmission = true;
			SubmitTask.SubmitTaskID = LiveTranslateJob->PermanentID;
		}
		else
		{
			UE_LOGF(LogRenderTrace, Warning, "Got submit event with unknown instigator translate job ID %llx, not linking.", SubmissionID);
		}
	}

	// We'll add subevents for all the submitted translate jobs. Since these don't have durations either, we'll just create
	// equal-sized subevents for the fake duration. Note that SubEventDuration will be infinity if there are no jobs, but
	// that's fine since the value is only used in the loop below, which will not be entered in that case.
	const double EventDuration = SubmitTask.EndTime - SubmitTask.StartTime;
	const double SubEventDuration = EventDuration / TranslateJobIDs.Num();
	double SubEventStartTime = SubmitTask.StartTime;

	for (uint64 TranslateJobID : TranslateJobIDs)
	{
		const FLiveTranslateJob* LiveTranslateJob = LiveTranslateJobs.Find(TranslateJobID);
		if (!LiveTranslateJob)
		{
			UE_LOGF(LogRenderTrace, Warning, "Got submit event for unknown translate job ID %llx.", TranslateJobID);
			continue;
		}

		const uint32 PermanentTranslateJobID = LiveTranslateJob->PermanentID;
		FRHITranslateTask& TranslateJob = Provider.EditRHITranslateTask(PermanentTranslateJobID);
		check(TranslateJob.Type == ERHITranslateTaskType::Translate);
		TranslateJob.NextPhaseTaskID = NewTaskEntry.Key;

		if (!LiveTranslateJob->bAddedToTimeline)
		{
			// This can happen early in the frame when the game thread is setting up the immediate command list and we don't get
			// finalize events for the translate jobs.
			if (TranslateJob.EndTime != 0.0)
			{
				AddTranslateTasksToTimeline(TranslateJob.ThreadID);
			}
			else
			{
				UE_LOGF(LogRenderTrace, Warning, "Got submit event translate job ID %llx which doesn't have an end time.", TranslateJobID);
			}
		}

		const double SubEventEndTime = FMath::Min(SubEventStartTime + SubEventDuration, SubmitTask.EndTime);

		FEventInterval& SubEvent = SubmitTask.ProcessedItems.Emplace_GetRef(PermanentTranslateJobID, SubEventStartTime, SubEventEndTime);
		Timeline.AppendBeginEvent(SubEvent.Start, FRenderTraceEvent(SubEvent.ItemID, NewTaskEntry.Key));
		Timeline.AppendEndEvent(SubEvent.End);

		LiveTranslateJobs.Remove(TranslateJobID);
		SubEventStartTime = SubEventEndTime;
	}

	Timeline.AppendEndEvent(SubmitTask.EndTime);
}

void FRenderTraceAnalyzer::ProcessAssociateContext(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	// Since platform contexts are pooled, we will only get this message once, when the pooled object is created. These tend to live forever,
	// so PlatformToRHIContext is not like other mapping tables, which are constantly updated as objects are created and destroyed. Its contents
	// persist for the entire duration of the analysis session, because we don't have explicit lifetime messages for contexts.
	const uint64 PlatformContextID = EventData.GetValue<uint64>("PlatformContextID");
	const uint64 RHIContextID = EventData.GetValue<uint64>("RHIContextID");

	const uint64* ExistingPlatformContextMapping = PlatformToRHIContext.Find(PlatformContextID);
	if (ExistingPlatformContextMapping)
	{
		if (*ExistingPlatformContextMapping != RHIContextID)
		{
			UE_LOGF(LogRenderTrace, Warning, "Got associate platform context event from platform context ID %llx to RHI context ID %llx, but the platform context is already associated with RHI context %llx.",
				PlatformContextID, RHIContextID, *ExistingPlatformContextMapping);
		}
		// Either way, we don't need to do anything.
		return;
	}

	PlatformToRHIContext.Add(PlatformContextID, RHIContextID);
}

void FRenderTraceAnalyzer::ProcessPayloadCreated(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 PayloadID = EventData.GetValue<uint64>("PayloadID");
	const uint8 Queue = EventData.GetValue<uint8>("Queue");

	const uint32* ExistingPermanentID = LivePlatformPayloads.Find(PayloadID);
	if (ExistingPermanentID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got payload created event for payload ID %llx, but the payload is already live. Ignoring.", PayloadID);
		return;
	}

	TPair<uint32, FPlatformPayload&> NewPayloadEntry = Provider.AddPlatformPayload(PayloadID, Timestamp, Queue);
	LivePlatformPayloads.Add(PayloadID, NewPayloadEntry.Key);
}

void FRenderTraceAnalyzer::ProcessAddPayload(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 PlatformContextID = EventData.GetValue<uint64>("PlatformContextID");
	const uint64 PayloadID = EventData.GetValue<uint64>("PayloadID");

	const uint32* ExistingPermanentPayloadID = LivePlatformPayloads.Find(PayloadID);
	if (!ExistingPermanentPayloadID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got add payload event for unknown payload ID %llx. Ignoring.", PayloadID);
		return;
	}

	FPlatformPayload& Payload = Provider.EditPlatformPayload(*ExistingPermanentPayloadID);
	if (Payload.PipeIdx > 1)
	{
		// We ignore copy queue payloads for now.
		return;
	}

	const uint64* ExistingPlatformContextMapping = PlatformToRHIContext.Find(PlatformContextID);
	if (!ExistingPlatformContextMapping)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got add payload event for unknown platform context ID %llx.", PlatformContextID);
		return;
	}

	const uint32* ExistingRHIContextMapping = LiveRHIContexts.Find(*ExistingPlatformContextMapping);
	if (!ExistingRHIContextMapping)
	{
		// It's only ok to have a missing mapping if this is happening during startup when the immediate command list is being set up.
		if (ThreadID != GameThreadID)
		{
			UE_LOGF(LogRenderTrace, Warning, "Got add payload event for platform context ID %llx which maps to RHI context ID %llx, but the RHI context isn't live.", PlatformContextID, *ExistingPlatformContextMapping);
		}
		return;
	}

	const uint32 TranslateJobID = *ExistingRHIContextMapping;
	FRHITranslateTask& TranslateJob = Provider.EditRHITranslateTask(TranslateJobID);

	FRHITranslateContext& TranslateContext = TranslateJob.Contexts[Payload.PipeIdx];
	if (TranslateContext.RHIContextID != *ExistingPlatformContextMapping)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got add payload event for platform context ID 0x%llx, payload ID 0x%llx, but the context on pipe %u is 0x%llx, not 0x%llx. Ignoring.",
			PlatformContextID, PayloadID, Payload.PipeIdx, TranslateContext.RHIContextID, *ExistingPlatformContextMapping);
		return;
	}

	TranslateContext.PayloadIDs.Emplace(*ExistingPermanentPayloadID);

	Payload.TranslateEvent.ParentItemID = TranslateJobID;
	Payload.TranslateEvent.ItemID = TranslateContext.PayloadIDs.Num() - 1;
}

void FRenderTraceAnalyzer::ProcessCommandListOp(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 PayloadID = EventData.GetValue<uint64>("PayloadID");
	const uint64 PlatformCmdListID = EventData.GetValue<uint64>("PlatformCmdListID");
	const uint8 Op = EventData.GetValue<uint8>("Op");

	FPlatformPayload* Payload = FindPlatformPayload(PayloadID, TEXT("command list op"), ThreadID);
	if (!Payload)
	{
		return;
	}

	switch (Op)
	{
	case CommandListOp_Open:
	{
		Payload->CmdLists.Emplace(PlatformCmdListID);
		break;
	}

	case CommandListOp_Close:
	{
		// Move the end of the payload to the end of this command list, but don't exceed the translate job lifetime if it's already set.
		Payload->EndTime = FMath::Max(Timestamp, Payload->EndTime);
		if (Payload->TranslateEvent.ParentItemID != INVALID_EVENT_ID)
		{
			FRHITranslateTask& TranslateJob = Provider.EditRHITranslateTask(Payload->TranslateEvent.ParentItemID);
			if (TranslateJob.EndTime != 0.0)
			{
				Payload->EndTime = FMath::Min(Payload->EndTime, TranslateJob.EndTime);
			}
		}
		break;
	}

	default:
	{
		UE_LOGF(LogRenderTrace, Warning, "Got command list open event for payload ID %llx, command list ID %llx with unknown op code %u.",
			PayloadID, PlatformCmdListID, Op);
		break;
	}
	}
}

void FRenderTraceAnalyzer::ProcessSyncPointOp(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 PayloadID = EventData.GetValue<uint64>("PayloadID");
	const uint64 SyncPointID = EventData.GetValue<uint64>("SyncPointID");
	const uint8 SyncPointType = EventData.GetValue<uint8>("SyncPointType");
	const uint8 Op = EventData.GetValue<uint8>("Op");

	FPlatformPayload* Payload = FindPlatformPayload(PayloadID, TEXT("sync point op"), ThreadID);
	if (!Payload)
	{
		return;
	}

	const uint32* ExistingSyncPointID = LiveSyncPoints.Find(SyncPointID);
	uint32 PermanentSyncPointID;
	if (!ExistingSyncPointID)
	{
		TPair<uint32, FSyncPoint&> NewSyncPoint = Provider.AddSyncPoint(SyncPointID, static_cast<ESyncPointType>(SyncPointType));
		PermanentSyncPointID = NewSyncPoint.Key;
		LiveSyncPoints.Add(SyncPointID, PermanentSyncPointID);
	}
	else
	{
		PermanentSyncPointID = *ExistingSyncPointID;
	}

	auto& SyncPointList = (Op == 0) ? Payload->WaitSyncPoints : Payload->SignalSyncPoints;
	SyncPointList.Add(PermanentSyncPointID);
}

void FRenderTraceAnalyzer::ProcessManualFenceOp(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 PayloadID = EventData.GetValue<uint64>("PayloadID");
	const uint64 FenceID = EventData.GetValue<uint64>("FenceID");
	const uint64 FenceVal = EventData.GetValue<uint64>("FenceVal");
	const uint8 Op = EventData.GetValue<uint8>("Op");

	FPlatformPayload* Payload = FindPlatformPayload(PayloadID, TEXT("manual fence op"), ThreadID);
	if (!Payload)
	{
		return;
	}

	const uint32* ExistingSyncPointID = ManualSyncPoints.Find(FenceID);
	uint32 PermanentSyncPointID;
	if (!ExistingSyncPointID)
	{
		TPair<uint32, FSyncPoint&> NewSyncPoint = Provider.AddSyncPoint(FenceID, ESyncPointType::Manual);
		PermanentSyncPointID = NewSyncPoint.Key;
		NewSyncPoint.Value.ResolvedValue = FenceVal;
		ManualSyncPoints.Add(FenceID, PermanentSyncPointID);
	}
	else
	{
		PermanentSyncPointID = *ExistingSyncPointID;
		FSyncPoint& ExistingSyncPoint = Provider.EditSyncPoint(PermanentSyncPointID);
		ExistingSyncPoint.ResolvedValue = FenceVal;
	}

	auto& SyncPointList = (Op == 0) ? Payload->WaitSyncPoints : Payload->SignalSyncPoints;
	SyncPointList.Add(PermanentSyncPointID);
}

void FRenderTraceAnalyzer::ProcessProcessSubmissionQueueEnter(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	if (PendingSubmissionBatchID != INVALID_EVENT_ID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got new submission batch event while the previous one is still open, merging.");
		return;
	}

	TPair<uint32, FSubmissionBatch&> NewSubmissionBatch = Provider.AddSubmissionBatch(ThreadID, Timestamp);
	PendingSubmissionBatchID = NewSubmissionBatch.Key;
}

// This macro creates Batch and SubmissionEvent variables for convenience.
#define ADD_SUBMIT_EVENT(Name, ...)\
	if (PendingSubmissionBatchID == INVALID_EVENT_ID)\
	{\
		UE_LOG(LogRenderTrace, Warning, TEXT("Got " #Name " submission event with no open batch, ignoring."));\
		return;\
	}\
	FSubmissionBatch& Batch = Provider.EditSubmissionBatch(PendingSubmissionBatchID);\
	check(Batch.EndTime == 0.0);\
	FSubmissionEvent& SubmissionEvent = Batch.Events.Emplace_GetRef(Timestamp, Queue);\
	SubmissionEvent.Data.Emplace<FSubmissionEvent::F##Name>(__VA_ARGS__)
	
void FRenderTraceAnalyzer::ProcessSubmitYieldOnUnresolvedSyncPoint(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint8 Queue = EventData.GetValue<uint8>("Queue");
	uint64 PayloadID = EventData.GetValue<uint64>("PayloadID");
	uint64 SyncPointID = EventData.GetValue<uint64>("SyncPointID");

	ADD_SUBMIT_EVENT(YieldSyncPoint, PayloadID, SyncPointID);
}

void FRenderTraceAnalyzer::ProcessSubmitYieldOnUnresolvedManualFence(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint8 Queue = EventData.GetValue<uint8>("Queue");
	uint64 PayloadID = EventData.GetValue<uint64>("PayloadID");
	uint64 FenceID = EventData.GetValue<uint64>("FenceID");
	uint64 FenceValue = EventData.GetValue<uint64>("FenceValue");

	ADD_SUBMIT_EVENT(YieldManualFence, PayloadID, FenceID, FenceValue);
}

void FRenderTraceAnalyzer::ProcessSubmitWaitQueueFence(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 PayloadID = EventData.GetValue<uint64>("PayloadID");
	const uint8 Queue = EventData.GetValue<uint8>("ExecutingQueue");
	const uint8 WaitOnQueue = EventData.GetValue<uint8>("WaitOnQueue");
	const uint64 Value = EventData.GetValue<uint64>("Value");

	ADD_SUBMIT_EVENT(WaitQueueFence, PayloadID, WaitOnQueue, Value);
}

void FRenderTraceAnalyzer::ProcessSubmitWaitManualFence(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 PayloadID = EventData.GetValue<uint64>("PayloadID");
	const uint8 Queue = EventData.GetValue<uint8>("ExecutingQueue");
	const uint64 FenceID = EventData.GetValue<uint64>("FenceID");
	const uint64 Value = EventData.GetValue<uint64>("Value");

	ADD_SUBMIT_EVENT(WaitManualFence, PayloadID, FenceID, Value);
}

void FRenderTraceAnalyzer::ProcessSubmitPlatformCommandLists(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint8 Queue = EventData.GetValue<uint8>("Queue");
	TConstArrayView<uint64> PayloadIDs = EventData.GetArrayView<uint64>("PayloadIDs");
	TConstArrayView<uint64> CmdListIDs = EventData.GetArrayView<uint64>("PlatformCmdListIDs");

	TArray<uint32> PayloadPermanentIDs;
	PayloadPermanentIDs.Reserve(PayloadIDs.Num());
	for (uint64 PayloadID : PayloadIDs)
	{
		const uint32* ExistingPermanentID = LivePlatformPayloads.Find(PayloadID);
		if (ExistingPermanentID)
		{
			PayloadPermanentIDs.Add(*ExistingPermanentID);
		}
	}

	ADD_SUBMIT_EVENT(Execute, PayloadPermanentIDs, CmdListIDs);

	for (uint32 PayloadID : PayloadPermanentIDs)
	{
		FPlatformPayload& Payload = Provider.EditPlatformPayload(PayloadID);
		Payload.ExecutionEvent.ParentItemID = PendingSubmissionBatchID;
		Payload.ExecutionEvent.ItemID = Batch.Events.Num() - 1;
	}
}

void FRenderTraceAnalyzer::ProcessSubmitSignalManualFence(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 PayloadID = EventData.GetValue<uint64>("PayloadID");
	const uint8 Queue = EventData.GetValue<uint8>("Queue");
	const uint64 FenceID = EventData.GetValue<uint64>("FenceID");
	const uint64 Value = EventData.GetValue<uint64>("Value");

	ADD_SUBMIT_EVENT(SignalManualFence, PayloadID, FenceID, Value);
}

void FRenderTraceAnalyzer::ProcessSubmitSignalQueueFence(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint64 PayloadID = EventData.GetValue<uint64>("PayloadID");
	const uint8 Queue = EventData.GetValue<uint8>("Queue");
	const uint64 Value = EventData.GetValue<uint64>("Value");

	ADD_SUBMIT_EVENT(SignalQueueFence, PayloadID, Value);
}

void FRenderTraceAnalyzer::ProcessSubmitResolveSyncPoint(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint8 Queue = EventData.GetValue<uint8>("Queue");
	const uint64 PayloadID = EventData.GetValue<uint64>("PayloadID");
	const uint64 SyncPointID = EventData.GetValue<uint64>("SyncPointID");
	const uint64 Value = EventData.GetValue<uint64>("Value");

	const uint32* PermanentSyncPointID = LiveSyncPoints.Find(SyncPointID);
	if (!PermanentSyncPointID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got resolve sync point event for unknown sync point ID %llx.", SyncPointID);
		return;
	}

	const uint32* PermanentPayloadID = LivePlatformPayloads.Find(PayloadID);
	if (!PermanentPayloadID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got resolve sync point event for sync point ID %llx with unknown payload ID %llx.", SyncPointID, PayloadID);
		return;
	}

	ADD_SUBMIT_EVENT(ResolveSyncPoint, *PermanentPayloadID, *PermanentSyncPointID, Value);

	FSyncPoint& SyncPoint = Provider.EditSyncPoint(*PermanentSyncPointID);
	check(SyncPoint.ResolvedValue == 0);
	SyncPoint.ResolvedValue = Value;
	SyncPoint.ResolvedByPayload = *PermanentPayloadID;

	FPlatformPayload& Payload = Provider.EditPlatformPayload(*PermanentPayloadID);
	Payload.ResolveSyncPointEvents.Emplace(Batch.Events.Num() - 1, PendingSubmissionBatchID);

	LiveSyncPoints.Remove(SyncPointID);
}

void FRenderTraceAnalyzer::EndPendingSubmissionBatch(double Timestamp, uint8 ExitStatus)
{
	check(PendingSubmissionBatchID != INVALID_EVENT_ID);

	FSubmissionBatch& Batch = Provider.EditSubmissionBatch(PendingSubmissionBatchID);
	check(Batch.EndTime == 0.0);
	Batch.EndTime = Timestamp;
	Batch.ExitStatus = ExitStatus;

	IRenderTraceProvider::TEventTimeline& Timeline = Provider.EditSubmissionQueueTimeline();
	Timeline.AppendBeginEvent(Batch.StartTime, FRenderTraceEvent(PendingSubmissionBatchID));
	for (int EventIdx = 0; EventIdx < Batch.Events.Num(); ++EventIdx)
	{
		const FSubmissionEvent& Event = Batch.Events[EventIdx];
		Timeline.AppendBeginEvent(Event.Timestamp, FRenderTraceEvent(EventIdx, PendingSubmissionBatchID));
		const double EventEndTime = EventIdx < Batch.Events.Num() - 1 ? Batch.Events[EventIdx + 1].Timestamp : Batch.EndTime;
		Timeline.AppendEndEvent(EventEndTime);
	}
	Timeline.AppendEndEvent(Batch.EndTime);

	PendingSubmissionBatchID = INVALID_EVENT_ID;
}

void FRenderTraceAnalyzer::ProcessProcessSubmissionQueueExit(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	if (PendingSubmissionBatchID == INVALID_EVENT_ID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got end submission batch event with no open batch, ignoring.");
		return;
	}

	const uint8 ExitStatus = EventData.GetValue<uint8>("Status");
	EndPendingSubmissionBatch(Timestamp, ExitStatus);
}

#undef ADD_SUBMIT_EVENT

void FRenderTraceAnalyzer::ProcessProcessInterruptQueueEnter(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	if (PendingInterruptWakeUpID != INVALID_EVENT_ID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got new interrupt wake up event while the previous one is still open, merging.");
		return;
	}

	TPair<uint32, FInterruptWakeUp&> NewWakeUp = Provider.AddInterruptWakeUp(ThreadID, Timestamp);
	PendingInterruptWakeUpID = NewWakeUp.Key;
}

void FRenderTraceAnalyzer::ProcessInterruptQueueFenceSignaled(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	const uint8 Queue = EventData.GetValue<uint8>("Queue");
	const uint64 PendingPayloadID = EventData.GetValue<uint64>("PendingPayloadID");
	const uint64 CurrentFenceValue = EventData.GetValue<uint64>("CurrentFenceValue");
	const uint64 LastCPUSignaledFenceValue = EventData.GetValue<uint64>("LastCPUSignaledFenceValue");

	if (PendingInterruptWakeUpID == INVALID_EVENT_ID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got fence signaled event with no open interrupt wake up, ignoring.");
		return;
	}

	const uint32* ExistingPermanentID = LivePlatformPayloads.Find(PendingPayloadID);
	if (!ExistingPermanentID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got fence signaled event for unknown payload 0x%llx, ignoring.", PendingPayloadID);
		return;
	}

	FInterruptWakeUp& WakeUp = Provider.EditInterruptWakeUp(PendingInterruptWakeUpID);
	WakeUp.SignalEvents.Emplace(Timestamp, Queue, *ExistingPermanentID, CurrentFenceValue, LastCPUSignaledFenceValue);

	FPlatformPayload& Payload = Provider.EditPlatformPayload(*ExistingPermanentID);
	Payload.InterruptEvent.ParentItemID = PendingInterruptWakeUpID;
	Payload.InterruptEvent.ItemID = WakeUp.SignalEvents.Num() - 1;

	LivePlatformPayloads.Remove(PendingPayloadID);
}

void FRenderTraceAnalyzer::EndPendingInterruptWakeUp(double Timestamp, uint8 ExitStatus)
{
	check(PendingInterruptWakeUpID != INVALID_EVENT_ID);

	FInterruptWakeUp& WakeUp = Provider.EditInterruptWakeUp(PendingInterruptWakeUpID);
	check(WakeUp.EndTime == 0.0);
	WakeUp.EndTime = Timestamp;
	WakeUp.ExitStatus = ExitStatus;

	IRenderTraceProvider::TEventTimeline& Timeline = Provider.EditInterruptTimeline();
	Timeline.AppendBeginEvent(WakeUp.StartTime, FRenderTraceEvent(PendingInterruptWakeUpID));
	for (int EventIdx = 0; EventIdx < WakeUp.SignalEvents.Num(); ++EventIdx)
	{
		const FInterruptFenceSignaledEvent& Event = WakeUp.SignalEvents[EventIdx];
		Timeline.AppendBeginEvent(Event.Timestamp, FRenderTraceEvent(EventIdx, PendingInterruptWakeUpID));
		const double EventEndTime = EventIdx < WakeUp.SignalEvents.Num() - 1 ? WakeUp.SignalEvents[EventIdx + 1].Timestamp : WakeUp.EndTime;
		Timeline.AppendEndEvent(EventEndTime);
	}
	Timeline.AppendEndEvent(WakeUp.EndTime);

	PendingInterruptWakeUpID = INVALID_EVENT_ID;
}

void FRenderTraceAnalyzer::ProcessProcessInterruptQueueExit(const FEventData& EventData, uint32 ThreadID, double Timestamp)
{
	if (PendingInterruptWakeUpID == INVALID_EVENT_ID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Got end interrupt wake up event with no open event, ignoring.");
		return;
	}

	const uint8 ExitStatus = EventData.GetValue<uint8>("Status");
	EndPendingInterruptWakeUp(Timestamp, ExitStatus);
}

bool FRenderTraceAnalyzer::CacheThreadIDs()
{
	// If everything is cached there's nothing we need to do.
	if (GameThreadID != 0 && RenderThreadID != 0 && RHIThreadID != 0)
	{
		return false;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);
	const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(Session);
	if (ThreadProvider.GetModCount() == LastThreadProviderModCount)
	{
		// The thread provider state hasn't changed since last time we checked, there's not point checking again.
		return false;
	}

	bool bThreadAdded = false;
	ThreadProvider.EnumerateThreads([this, &bThreadAdded](const TraceServices::FThreadInfo& ThreadInfo)
	{
		if (FCString::Strcmp(ThreadInfo.Name, TEXT("GameThread")) == 0)
		{
			GameThreadID = ThreadInfo.Id;
		}
		else if (FCString::Strncmp(ThreadInfo.Name, TEXT("RenderThread"), 12) == 0)
		{
			if (RenderThreadID == 0)
			{
				RenderThreadID = ThreadInfo.Id;
				bThreadAdded = true;
			}
		}
		else if (FCString::Strncmp(ThreadInfo.Name, TEXT("RHIThread"), 9) == 0)
		{
			if (RHIThreadID == 0)
			{
				RHIThreadID = ThreadInfo.Id;
				bThreadAdded = true;
			}
		}
	});

	LastThreadProviderModCount = ThreadProvider.GetModCount();
	return bThreadAdded;
}

bool FRenderTraceAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& EventContext)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FRenderTraceAnalyzer"));

	const FEventData& EventData = EventContext.EventData;
	const uint32 ThreadID = EventContext.ThreadInfo.GetId();
	const double Timestamp = EventContext.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));

	bool bThreadAdded = CacheThreadIDs();

	FinalTimestamp = FMath::Max(Timestamp, FinalTimestamp);

	double CurrentDuration;
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);
		CurrentDuration = Session.GetDurationSeconds();
	}

	if (FinalTimestamp - CurrentDuration > 5.0)
	{
		TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
		Session.UpdateDurationSeconds(FinalTimestamp);
	}

	TraceServices::FProviderEditScopeLock ProviderEditScope(Provider);

	if (bThreadAdded)
	{
		// Make sure the render thread and RHI thread timelines come first in their respective categories. It's fine
		// to call these functions multiple times when the RT and RHIT appear at different points in time since they
		// will only add a timeline the first time they're called.
		if (RenderThreadID != 0)
		{
			Provider.EditRDGThreadTimeline(RenderThreadID);
		}

		if (RHIThreadID != 0)
		{
			Provider.EditRHITranslateThreadTimeline(RHIThreadID);
		}
	}

#define DISPATCH_EVENT(Name) case Packet_##Name: Process##Name(EventData, ThreadID, Timestamp); break

	switch (RouteId)
	{
	DISPATCH_EVENT(CommandListCreated);
	DISPATCH_EVENT(CommandListDetach);
	DISPATCH_EVENT(CommandListFinishRecording);
	DISPATCH_EVENT(CommandListDestroyed);
	DISPATCH_EVENT(AddDispatchPassCommandList);
	DISPATCH_EVENT(SubmitCommandLists);
	DISPATCH_EVENT(SplitTranslateJob);
	DISPATCH_EVENT(DispatchCommandList);
	DISPATCH_EVENT(BeginTranslateCommandList);
	DISPATCH_EVENT(ActivatePipelines);
	DISPATCH_EVENT(SetTranslateContext);
	DISPATCH_EVENT(EndTranslateCommandList);
	DISPATCH_EVENT(FinalizeTranslateJob);
	DISPATCH_EVENT(SubmitTranslateJobs);
	DISPATCH_EVENT(AssociateContext);
	DISPATCH_EVENT(PayloadCreated);
	DISPATCH_EVENT(AddPayload);
	DISPATCH_EVENT(CommandListOp);
	DISPATCH_EVENT(SyncPointOp);
	DISPATCH_EVENT(ManualFenceOp);
	DISPATCH_EVENT(ProcessSubmissionQueueEnter);
	DISPATCH_EVENT(SubmitYieldOnUnresolvedSyncPoint);
	DISPATCH_EVENT(SubmitYieldOnUnresolvedManualFence);
	DISPATCH_EVENT(SubmitWaitQueueFence);
	DISPATCH_EVENT(SubmitWaitManualFence);
	DISPATCH_EVENT(SubmitPlatformCommandLists);
	DISPATCH_EVENT(SubmitSignalManualFence);
	DISPATCH_EVENT(SubmitSignalQueueFence);
	DISPATCH_EVENT(SubmitResolveSyncPoint);
	DISPATCH_EVENT(ProcessSubmissionQueueExit);
	DISPATCH_EVENT(ProcessInterruptQueueEnter);
	DISPATCH_EVENT(InterruptQueueFenceSignaled);
	DISPATCH_EVENT(ProcessInterruptQueueExit);

	case Packet_RDGPassNameSpec: ProcessRDGPassNameSpec(EventData, ThreadID, Timestamp); break;
	case Packet_BeginExecuteRDGPass: ProcessBeginExecuteRDGPass(EventData, ThreadID, Timestamp, false); break;
	case Packet_BeginExecuteDispatchPass: ProcessBeginExecuteRDGPass(EventData, ThreadID, Timestamp, true); break;
	case Packet_EndExecuteRDGPass: ProcessEndExecuteRDGPass(EventData, ThreadID, Timestamp, false); break;
	case Packet_EndExecuteDispatchPass: ProcessEndExecuteRDGPass(EventData, ThreadID, Timestamp, true); break;
	}

#undef DISPATCH_EVENT

	return true;
}

void FRenderTraceAnalyzer::OnAnalysisEnd()
{
	{
		TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
		Session.UpdateDurationSeconds(FinalTimestamp);
	}

	TraceServices::FProviderEditScopeLock ProviderEditScope(Provider);

	// Close pending events with an infinity end timestamp so that Insights knows that they never ended.
	const double Timestamp = std::numeric_limits<double>::infinity();

	// The trace might be truncated, so we need to close any pending objects.
	if (!LiveRDGPasses.IsEmpty())
	{
		UE_LOGF(LogRenderTrace, Warning, "Analysis ended with %u pending RDG passes, force-closing them.", LiveRDGPasses.Num());
		for (const auto& Pair : LiveRDGPasses)
		{
			EndRDGPassExecute(Pair.Value, Timestamp);
		}
		LiveRDGPasses.Empty();
	}

	if (!LiveCommandLists.IsEmpty())
	{
		UE_LOGF(LogRenderTrace, Warning, "Analysis ended with %u pending RHI command lists, force-closing them.", LiveCommandLists.Num());
		for (const auto& Pair : LiveCommandLists)
		{
			EndCommandList(Pair.Value, Timestamp);
		}
		LiveCommandLists.Empty();
	}

	if (!LiveTranslateJobs.IsEmpty())
	{
		UE_LOGF(LogRenderTrace, Warning, "Analysis ended with %u pending translate jobs, force-closing them.", LiveTranslateJobs.Num());
		for (const auto& Pair : LiveTranslateJobs)
		{
			EndTranslateTask(Pair.Value.PermanentID, Timestamp);
		}
		LiveTranslateJobs.Empty();
	}

	if (!LiveRHIContexts.IsEmpty())
	{
		UE_LOGF(LogRenderTrace, Warning, "Analysis ended with %u pending RHI contexts.", LiveRHIContexts.Num());
		LiveRHIContexts.Empty();
	}

	PlatformToRHIContext.Empty();

	if (!PendingRTSubmitIDs.IsEmpty())
	{
		UE_LOGF(LogRenderTrace, Warning, "Analysis ended with %u pending render thread submits.", PendingRTSubmitIDs.Num());
		PendingRTSubmitIDs.Empty();
	}

	if (!LivePlatformPayloads.IsEmpty())
	{
		UE_LOGF(LogRenderTrace, Warning, "Analysis ended with %u pending platform payloads.", LivePlatformPayloads.Num());
		LivePlatformPayloads.Empty();
	}

	if (!LiveSyncPoints.IsEmpty())
	{
		UE_LOGF(LogRenderTrace, Warning, "Analysis ended with %u pending sync points.", LiveSyncPoints.Num());
		LiveSyncPoints.Empty();
	}

	ManualSyncPoints.Empty();

	if (PendingSubmissionBatchID != INVALID_EVENT_ID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Analysis ended a pending submission batch, force-closing it.");
		EndPendingSubmissionBatch(Timestamp, 0);
	}

	if (PendingInterruptWakeUpID != INVALID_EVENT_ID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Analysis ended a pending interrupt wake up, force-closing it.");
		EndPendingInterruptWakeUp(Timestamp, 0);
	}
}

} //namespace RenderTraceInsights
} //namespace UE
