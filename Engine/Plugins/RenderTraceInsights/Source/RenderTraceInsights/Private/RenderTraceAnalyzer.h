// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "IRenderTraceProvider.h"

namespace TraceServices { class IAnalysisSession; }

namespace UE
{
namespace RenderTraceInsights
{ 

class FRenderTraceAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FRenderTraceAnalyzer(TraceServices::IAnalysisSession& InSession, IEditableRenderTraceProvider& InProvider);

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& EventContext) override;

private:
	enum : uint16
	{
		Packet_CommandListCreated,
		Packet_CommandListDetach,
		Packet_CommandListFinishRecording,
		Packet_CommandListDestroyed,
		Packet_RDGPassNameSpec,
		Packet_BeginExecuteRDGPass,
		Packet_EndExecuteRDGPass,
		Packet_BeginExecuteDispatchPass,
		Packet_AddDispatchPassCommandList,
		Packet_EndExecuteDispatchPass,
		Packet_SubmitCommandLists,
		Packet_SplitTranslateJob,
		Packet_DispatchCommandList,
		Packet_BeginTranslateCommandList,
		Packet_ActivatePipelines,
		Packet_SetTranslateContext,
		Packet_EndTranslateCommandList,
		Packet_FinalizeTranslateJob,
		Packet_SubmitTranslateJobs,
		Packet_AssociateContext,
		Packet_PayloadCreated,
		Packet_AddPayload,
		Packet_CommandListOp,
		Packet_SyncPointOp,
		Packet_ManualFenceOp,
		Packet_ProcessSubmissionQueueEnter,
		Packet_SubmitYieldOnUnresolvedSyncPoint,
		Packet_SubmitYieldOnUnresolvedManualFence,
		Packet_SubmitWaitQueueFence,
		Packet_SubmitWaitManualFence,
		Packet_SubmitPlatformCommandLists,
		Packet_SubmitSignalManualFence,
		Packet_SubmitSignalQueueFence,
		Packet_SubmitResolveSyncPoint,
		Packet_ProcessSubmissionQueueExit,
		Packet_ProcessInterruptQueueEnter,
		Packet_InterruptQueueFenceSignaled,
		Packet_ProcessInterruptQueueExit,
	};

	struct FLiveTranslateJob
	{
		explicit FLiveTranslateJob(uint32 InPermanentID) : PermanentID(InPermanentID) {}
		uint32 PermanentID;
		bool bAddedToTimeline = false;
	};

	TraceServices::IAnalysisSession& Session;
	IEditableRenderTraceProvider& Provider;

	uint32 GameThreadID = 0;
	uint32 RenderThreadID = 0;
	uint32 RHIThreadID = 0;
	TMap<uint64, uint32> LiveRDGPasses;
	TMap<uint64, uint32> LiveCommandLists;
	TMap<uint64, FLiveTranslateJob> LiveTranslateJobs;
	TMap<uint64, uint32> LiveRHIContexts;
	TMap<uint64, uint64> PlatformToRHIContext;
	TMap<uint32, uint32> PendingRTSubmitIDs;
	TMap<uint64, uint32> LivePlatformPayloads;
	TMap<uint64, uint32> LiveSyncPoints;
	TMap<uint64, uint32> ManualSyncPoints;
	uint32 PendingSubmissionBatchID = INVALID_EVENT_ID;
	uint32 PendingInterruptWakeUpID = INVALID_EVENT_ID;
	double FinalTimestamp = 0.0;
	uint64 LastThreadProviderModCount = 0;
	TMap<uint32, const TCHAR*> PassNameLookup;

	void BeginCmdListExecute(uint64 CmdListAppID, uint32 PermanentPassID, double Timestamp);
	void EndRDGPassExecute(uint32 PermanentPassID, double Timestamp);
	void AddCommandListToTimeline(uint32 PermanentCmdListID);
	void EndCommandList(uint32 PermanentCmdListID, double Timestamp);
	void AddTranslateTasksToTimeline(uint32 ThreadID);
	void EndTranslateTask(uint32 PermanentTaskID, double Timestamp);
	void EndPendingSubmissionBatch(double Timestamp, uint8 ExitStatus);
	void EndPendingInterruptWakeUp(double Timestamp, uint8 ExitStatus);
	FRHITranslateTask* GetTranslateJobForCmdList(uint64 CmdListID, const TCHAR* EventName, uint32 ThreadID, uint32* OutTranslateJobID);
	FPlatformPayload* FindPlatformPayload(uint64 PayloadID, const TCHAR* EventName, uint32 ThreadID);
	bool CacheThreadIDs();

#define DECLARE_EVENT_PROCESSING_FUNC(EventName) void Process##EventName(const FEventData& EventData, uint32 ThreadID, double Timestamp)
	DECLARE_EVENT_PROCESSING_FUNC(CommandListCreated);
	DECLARE_EVENT_PROCESSING_FUNC(CommandListDetach);
	DECLARE_EVENT_PROCESSING_FUNC(CommandListFinishRecording);
	DECLARE_EVENT_PROCESSING_FUNC(CommandListDestroyed);
	DECLARE_EVENT_PROCESSING_FUNC(AddDispatchPassCommandList);
	DECLARE_EVENT_PROCESSING_FUNC(SubmitCommandLists);
	DECLARE_EVENT_PROCESSING_FUNC(SplitTranslateJob);
	DECLARE_EVENT_PROCESSING_FUNC(DispatchCommandList);
	DECLARE_EVENT_PROCESSING_FUNC(BeginTranslateCommandList);
	DECLARE_EVENT_PROCESSING_FUNC(ActivatePipelines);
	DECLARE_EVENT_PROCESSING_FUNC(SetTranslateContext);
	DECLARE_EVENT_PROCESSING_FUNC(EndTranslateCommandList);
	DECLARE_EVENT_PROCESSING_FUNC(FinalizeTranslateJob);
	DECLARE_EVENT_PROCESSING_FUNC(SubmitTranslateJobs);
	DECLARE_EVENT_PROCESSING_FUNC(AssociateContext);
	DECLARE_EVENT_PROCESSING_FUNC(PayloadCreated);
	DECLARE_EVENT_PROCESSING_FUNC(AddPayload);
	DECLARE_EVENT_PROCESSING_FUNC(CommandListOp);
	DECLARE_EVENT_PROCESSING_FUNC(SyncPointOp);
	DECLARE_EVENT_PROCESSING_FUNC(ManualFenceOp);
	DECLARE_EVENT_PROCESSING_FUNC(ProcessSubmissionQueueEnter);
	DECLARE_EVENT_PROCESSING_FUNC(SubmitYieldOnUnresolvedSyncPoint);
	DECLARE_EVENT_PROCESSING_FUNC(SubmitYieldOnUnresolvedManualFence);
	DECLARE_EVENT_PROCESSING_FUNC(SubmitWaitQueueFence);
	DECLARE_EVENT_PROCESSING_FUNC(SubmitWaitManualFence);
	DECLARE_EVENT_PROCESSING_FUNC(SubmitPlatformCommandLists);
	DECLARE_EVENT_PROCESSING_FUNC(SubmitSignalManualFence);
	DECLARE_EVENT_PROCESSING_FUNC(SubmitSignalQueueFence);
	DECLARE_EVENT_PROCESSING_FUNC(SubmitResolveSyncPoint);
	DECLARE_EVENT_PROCESSING_FUNC(ProcessSubmissionQueueExit);
	DECLARE_EVENT_PROCESSING_FUNC(ProcessInterruptQueueEnter);
	DECLARE_EVENT_PROCESSING_FUNC(InterruptQueueFenceSignaled);
	DECLARE_EVENT_PROCESSING_FUNC(ProcessInterruptQueueExit);
#undef DECLARE_EVENT_PROCESSING_FUNC

	// These are shared between regular and dispatch RDG passes so they have different signatures from the other event processing methods.
	void ProcessBeginExecuteRDGPass(const FEventData& EventData, uint32 ThreadID, double Timestamp, bool bIsDispatch);
	void ProcessEndExecuteRDGPass(const FEventData& EventData, uint32 ThreadID, double Timestamp, bool bIsDispatch);
	void ProcessRDGPassNameSpec(const FEventData& EventData, uint32 ThreadID, double Timestamp);
};

} //namespace RenderTraceInsights
} //namespace UE
