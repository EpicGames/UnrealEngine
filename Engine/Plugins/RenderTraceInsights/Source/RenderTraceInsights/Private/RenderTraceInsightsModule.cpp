// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderTraceInsightsModule.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"


namespace UE
{
namespace RenderTraceInsights
{

FRenderTraceInsightsModule& FRenderTraceInsightsModule::Get()
{
	return FModuleManager::LoadModuleChecked<FRenderTraceInsightsModule>("RenderTraceInsights");
}

void FRenderTraceInsightsModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &TraceModule);
	IModularFeatures::Get().RegisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);
}

void FRenderTraceInsightsModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &TraceModule);
	IModularFeatures::Get().UnregisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);
}

} //namespace RenderTraceInsights
} //namespace UE

IMPLEMENT_MODULE(UE::RenderTraceInsights::FRenderTraceInsightsModule, RenderTraceInsights);
