// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#include "RenderTraceModule.h"
#include "RenderTraceTimingViewExtender.h"

namespace UE
{
namespace RenderTraceInsights
{

class FRenderTraceInsightsModule : public IModuleInterface
{
public:
	static FRenderTraceInsightsModule& Get();

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

private:
	FRenderTraceModule TraceModule;
	FRenderTraceTimingViewExtender TimingViewExtender;
};

} //namespace RenderTraceInsights
} //namespace UE
