// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderTraceModule.h"
#include "RenderTraceProvider.h"
#include "RenderTraceAnalyzer.h"

namespace UE
{
namespace RenderTraceInsights
{

DEFINE_LOG_CATEGORY(LogRenderTrace);

FName FRenderTraceModule::ModuleName("TraceModule_RenderTrace");

void FRenderTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("RenderTrace");
}

void FRenderTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	TSharedPtr<FRenderTraceProvider> Provider = MakeShared<FRenderTraceProvider>(InSession);
	InSession.AddProvider(FRenderTraceProvider::ProviderName, Provider);

	InSession.AddAnalyzer(new FRenderTraceAnalyzer(InSession, *Provider));
}

void FRenderTraceModule::GetLoggers(TArray<const TCHAR *>& OutLoggers)
{
	OutLoggers.Add(TEXT("RenderTrace"));
}

} //namespace RenderTraceInsights
} //namespace UE
