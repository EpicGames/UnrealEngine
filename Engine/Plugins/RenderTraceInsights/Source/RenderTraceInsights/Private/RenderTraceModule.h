// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/ModuleService.h"

namespace UE
{
namespace RenderTraceInsights
{

DECLARE_LOG_CATEGORY_EXTERN(LogRenderTrace, Log, All);

class FRenderTraceModule : public TraceServices::IModule
{
public:
	//~ Begin TraceServices::IModule interface
	virtual void GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo) override;
	virtual void OnAnalysisBegin(TraceServices::IAnalysisSession& Session) override;
	virtual void GetLoggers(TArray<const TCHAR *>& OutLoggers) override;
	virtual const TCHAR* GetCommandLineArgument() override { return TEXT("rendertrace"); }
	//~ End TraceServices::IModule interface

private:
	static FName ModuleName;
};

} //namespace RenderTraceInsights
} //namespace UE
