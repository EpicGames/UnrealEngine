// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderTraceProvider.h"
#include "RenderTraceModule.h"

namespace UE
{
namespace RenderTraceInsights
{

thread_local TraceServices::FProviderLock::FThreadLocalState GRenderTraceProviderLockState;
FName FRenderTraceProvider::ProviderName("RenderTraceProvider");

FName GetRenderTraceProviderName()
{
	return FRenderTraceProvider::ProviderName;
}

const IRenderTraceProvider* ReadRenderTraceProvider(const TraceServices::IAnalysisSession& Session)
{
	return Session.ReadProvider<IRenderTraceProvider>(FRenderTraceProvider::ProviderName);
}

IEditableRenderTraceProvider* EditRenderTraceProvider(TraceServices::IAnalysisSession& Session)
{
	return Session.EditProvider<IEditableRenderTraceProvider>(FRenderTraceProvider::ProviderName);
}

FRenderTraceProvider::FRenderTraceProvider(TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
{
	RTSubmissionTimeline = MakeUnique<TEventTimeline>(Session.GetLinearAllocator());
	RHISubmissionTimeline = MakeUnique<TEventTimeline>(Session.GetLinearAllocator());
	SubmissionQueueTimeline = MakeUnique<TEventTimeline>(Session.GetLinearAllocator());
	InterruptTimeline = MakeUnique<TEventTimeline>(Session.GetLinearAllocator());
}

int32 FRenderTraceProvider::GetNumCommandLists() const
{
	ReadAccessCheck();
	return CommandLists.Num();
}

const FCommandListInstance& FRenderTraceProvider::GetCommandList(uint32 ID) const
{
	ReadAccessCheck();
	return CommandLists[ID];
}

int32 FRenderTraceProvider::GetNumCommandListTimelines() const
{
	ReadAccessCheck();
	return CommandListTimelines.Num();
}

void FRenderTraceProvider::ReadCommandListTimelines(TFunctionRef<void(uint32, const TEventTimeline&)> Callback) const
{
	ReadAccessCheck();
	for (uint32 i = 0; i < (uint32)CommandListTimelines.Num(); ++i)
	{
		Callback(i, *CommandListTimelines[i]);
	}
}

TPair<uint32, FCommandListInstance&> FRenderTraceProvider::AddCommandList(uint64 AppID, double Timestamp, ECommandListType Type)
{
	EditAccessCheck();
	const uint32 ID = CommandLists.Num();
	FCommandListInstance& NewCmdList = CommandLists.Emplace_GetRef(AppID, Timestamp, Type);
	return TPair<uint32, FCommandListInstance&>(ID, NewCmdList);
}

FCommandListInstance& FRenderTraceProvider::EditCommandList(uint32 ID)
{
	EditAccessCheck();
	return CommandLists[ID];
}

void FRenderTraceProvider::EnumerateCommandListsForEdit(TFunctionRef<void(uint32, FCommandListInstance&)> Callback)
{
	EditAccessCheck();
	for (uint32 i = 0; i < (uint32)CommandLists.Num(); ++i)
	{
		Callback(i, CommandLists[i]);
	}
}

TPair<uint32, FRenderTraceProvider::TEventTimeline&> FRenderTraceProvider::AddCommandListTimeline()
{
	EditAccessCheck();
	++TimelinesModCount;
	uint32 Index = CommandListTimelines.Num();
	TUniquePtr<TEventTimeline>& Timeline = CommandListTimelines.Emplace_GetRef(MakeUnique<TEventTimeline>(Session.GetLinearAllocator()));
	return TPair<uint32, TEventTimeline&>(Index, *Timeline);
}

FRenderTraceProvider::TEventTimeline& FRenderTraceProvider::EditCommandListTimeline(uint32 Index)
{
	EditAccessCheck();
	return *CommandListTimelines[Index];
}

int32 FRenderTraceProvider::GetNumRDGPasses() const
{
	ReadAccessCheck();
	return RDGPasses.Num();
}

const FRDGPassInstance& FRenderTraceProvider::GetRDGPass(uint32 ID) const
{
	ReadAccessCheck();
	return RDGPasses[ID];
}

void FRenderTraceProvider::ReadRDGTimelines(TFunctionRef<void(uint32, uint32, const TEventTimeline&)> Callback) const
{
	ReadAccessCheck();
	for (uint32 i = 0; i < (uint32)RDGTimelines.Num(); ++i)
	{
		Callback(i, RDGTimelines[i].ThreadID, *RDGTimelines[i].Timeline);
	}
}

const FRenderTraceProvider::TEventTimeline* FRenderTraceProvider::GetRenderThreadSubmissionTimeline() const
{
	ReadAccessCheck();
	return RTSubmissionTimeline.Get();
}

TPair<uint32, FRDGPassInstance&> FRenderTraceProvider::AddRDGPass(const TCHAR* Name, uint32 ThreadID, double Timestamp, ERDGPassType Type)
{
	EditAccessCheck();
	const uint32 ID = RDGPasses.Num();
	FRDGPassInstance& NewPass = RDGPasses.Emplace_GetRef(Name, ThreadID, Timestamp, Type);
	return TPair<uint32, FRDGPassInstance&>(ID, NewPass);
}

FRDGPassInstance& FRenderTraceProvider::EditRDGPass(uint32 ID)
{
	EditAccessCheck();
	return RDGPasses[ID];
}

FRenderTraceProvider::TEventTimeline& FRenderTraceProvider::EditRDGThreadTimeline(uint32 ThreadID)
{
	EditAccessCheck();

	uint32* ExistingIndex = RDGThreadToTimeline.Find(ThreadID);
	if (ExistingIndex)
	{
		return *RDGTimelines[*ExistingIndex].Timeline;
	}

	FThreadTimeline& NewTimeline = RDGTimelines.Emplace_GetRef(ThreadID, Session.GetLinearAllocator());
	RDGThreadToTimeline.Add(ThreadID, RDGTimelines.Num() - 1);
	++TimelinesModCount;
	return *NewTimeline.Timeline;
}

FRenderTraceProvider::TEventTimeline& FRenderTraceProvider::EditRenderThreadSubmissionTimeline()
{
	EditAccessCheck();
	return *RTSubmissionTimeline;
}

int32 FRenderTraceProvider::GetNumRHITranslateTasks() const
{
	ReadAccessCheck();
	return RHITranslateTasks.Num();
}

const FRHITranslateTask& FRenderTraceProvider::GetRHITranslateTask(uint32 ID) const
{
	ReadAccessCheck();
	return RHITranslateTasks[ID];
}

uint32 FRenderTraceProvider::FindRHITaskByPredicate(uint32 StartAtTaskID, double TimeRange, TFunctionRef<int(const FRHITranslateTask&)> Pred) const
{
	ReadAccessCheck();

	if (StartAtTaskID >= (uint32)RHITranslateTasks.Num())
	{
		return INVALID_EVENT_ID;
	}

	const double StartTaskTime = RHITranslateTasks[StartAtTaskID].StartTime;
	const double MinTime = StartTaskTime - TimeRange;
	const double MaxTime = StartTaskTime + TimeRange;
	int32 LeftIdx = (int32)StartAtTaskID - 1;
	int32 RightIdx = (int32)StartAtTaskID + 1;
	bool bLeftStop = false, bRightStop = false;

	while (!bLeftStop || !bRightStop)
	{
		if (!bLeftStop && LeftIdx >= 0 && RHITranslateTasks[LeftIdx].StartTime >= MinTime)
		{
			switch (Pred(RHITranslateTasks[LeftIdx]))
			{
			case -1: bLeftStop = true; break;
			case 0: --LeftIdx; break;
			case 1: return LeftIdx;
			}
		}
		else
		{
			bLeftStop = true;
		}

		if (!bRightStop && RightIdx < RHITranslateTasks.Num() && RHITranslateTasks[RightIdx].StartTime <= MaxTime)
		{
			switch (Pred(RHITranslateTasks[RightIdx]))
			{
			case -1: bRightStop = true; break;
			case 0: ++RightIdx; break;
			case 1: return RightIdx;
			}
		}
		else
		{
			bRightStop = true;
		}
	}

	return INVALID_EVENT_ID;
}

void FRenderTraceProvider::EnumerateRHITranslateTimelines(TFunctionRef<void(uint32, uint32, const TEventTimeline&)> Callback) const
{
	ReadAccessCheck();
	for (uint32 i = 0; i < (uint32)RHITranslateTimelines.Num(); ++i)
	{
		Callback(i, RHITranslateTimelines[i].ThreadID, *RHITranslateTimelines[i].Timeline);
	}
}

const FRenderTraceProvider::TEventTimeline* FRenderTraceProvider::GetRHISubmissionTimeline() const
{
	ReadAccessCheck();
	return RHISubmissionTimeline.Get();
}

TPair<uint32, FRHITranslateTask&> FRenderTraceProvider::AddRHITranslateTask(uint64 AppID, ERHITranslateTaskType Type, uint32 ThreadID, double Timestamp)
{
	EditAccessCheck();
	const uint32 ID = RHITranslateTasks.Num();
	FRHITranslateTask& NewTask = RHITranslateTasks.Emplace_GetRef(AppID, Type, ThreadID, Timestamp);
	return TPair<uint32, FRHITranslateTask&>(ID, NewTask);
}

FRHITranslateTask& FRenderTraceProvider::EditRHITranslateTask(uint32 ID)
{
	EditAccessCheck();
	return RHITranslateTasks[ID];
}

FRenderTraceProvider::TEventTimeline& FRenderTraceProvider::EditRHITranslateThreadTimeline(uint32 ThreadID)
{
	EditAccessCheck();

	uint32* ExistingIndex = RHIThreadToTimeline.Find(ThreadID);
	if (ExistingIndex)
	{
		return *RHITranslateTimelines[*ExistingIndex].Timeline;
	}

	FThreadTimeline& NewTimeline = RHITranslateTimelines.Emplace_GetRef(ThreadID, Session.GetLinearAllocator());
	RHIThreadToTimeline.Add(ThreadID, RHITranslateTimelines.Num() - 1);
	++TimelinesModCount;
	return *NewTimeline.Timeline;
}

FRenderTraceProvider::TEventTimeline& FRenderTraceProvider::EditRHISubmissionTimeline()
{
	EditAccessCheck();
	return *RHISubmissionTimeline;
}

int32 FRenderTraceProvider::GetNumPlatformPayloads() const
{
	ReadAccessCheck();
	return PlatformPayloads.Num();
}

const FPlatformPayload& FRenderTraceProvider::GetPlatformPayload(uint32 ID) const
{
	ReadAccessCheck();
	return PlatformPayloads[ID];
}

TPair<uint32, FPlatformPayload&> FRenderTraceProvider::AddPlatformPayload(uint64 InAppID, double InStartTime, uint8 InPipeIdx)
{
	EditAccessCheck();
	const uint32 ID = PlatformPayloads.Num();
	FPlatformPayload& NewPayload = PlatformPayloads.Emplace_GetRef(InAppID, InStartTime, InPipeIdx);
	return TPair<uint32, FPlatformPayload&>(ID, NewPayload);
}

FPlatformPayload& FRenderTraceProvider::EditPlatformPayload(uint32 ID)
{
	EditAccessCheck();
	return PlatformPayloads[ID];
}

int32 FRenderTraceProvider::GetNumSyncPoints() const
{
	ReadAccessCheck();
	return SyncPoints.Num();
}

const FSyncPoint& FRenderTraceProvider::GetSyncPoint(uint32 ID) const
{
	ReadAccessCheck();
	return SyncPoints[ID];
}

TPair<uint32, FSyncPoint&> FRenderTraceProvider::AddSyncPoint(uint64 InAppID, ESyncPointType InType)
{
	EditAccessCheck();
	const uint32 ID = SyncPoints.Num();
	FSyncPoint& NewSyncPoint = SyncPoints.Emplace_GetRef(InAppID, InType);
	return TPair<uint32, FSyncPoint&>(ID, NewSyncPoint);
}

FSyncPoint& FRenderTraceProvider::EditSyncPoint(uint32 ID)
{
	EditAccessCheck();
	return SyncPoints[ID];
}

int32 FRenderTraceProvider::GetNumSubmissionBatches() const
{
	ReadAccessCheck();
	return SubmissionBatches.Num();
}

const FSubmissionBatch& FRenderTraceProvider::GetSubmissionBatch(uint32 ID) const
{
	ReadAccessCheck();
	return SubmissionBatches[ID];
}

const FRenderTraceProvider::TEventTimeline* FRenderTraceProvider::GetSubmissionQueueTimeline() const
{
	ReadAccessCheck();
	return SubmissionQueueTimeline.Get();
}

TPair<uint32, FSubmissionBatch&> FRenderTraceProvider::AddSubmissionBatch(uint32 ThreadID, double Timestamp)
{
	EditAccessCheck();
	const uint32 ID = SubmissionBatches.Num();
	FSubmissionBatch& NewBatch = SubmissionBatches.Emplace_GetRef(ThreadID, Timestamp);
	return TPair<uint32, FSubmissionBatch&>(ID, NewBatch);
}

FSubmissionBatch& FRenderTraceProvider::EditSubmissionBatch(uint32 ID)
{
	EditAccessCheck();
	return SubmissionBatches[ID];
}

FRenderTraceProvider::TEventTimeline& FRenderTraceProvider::EditSubmissionQueueTimeline()
{
	EditAccessCheck();
	return *SubmissionQueueTimeline;
}

int32 FRenderTraceProvider::GetNumInterruptWakeUps() const
{
	ReadAccessCheck();
	return InterruptWakeUps.Num();
}

const FInterruptWakeUp& FRenderTraceProvider::GetInterruptWakeUp(uint32 ID) const
{
	ReadAccessCheck();
	return InterruptWakeUps[ID];
}

const FRenderTraceProvider::TEventTimeline* FRenderTraceProvider::GetInterruptTimeline() const
{
	ReadAccessCheck();
	return InterruptTimeline.Get();
}

TPair<uint32, FInterruptWakeUp&> FRenderTraceProvider::AddInterruptWakeUp(uint32 ThreadID, double Timestamp)
{
	EditAccessCheck();
	const uint32 ID = InterruptWakeUps.Num();
	FInterruptWakeUp& NewWakeUp = InterruptWakeUps.Emplace_GetRef(ThreadID, Timestamp);
	return TPair<uint32, FInterruptWakeUp&>(ID, NewWakeUp);
}

FInterruptWakeUp& FRenderTraceProvider::EditInterruptWakeUp(uint32 ID)
{
	EditAccessCheck();
	return InterruptWakeUps[ID];
}

FRenderTraceProvider::TEventTimeline& FRenderTraceProvider::EditInterruptTimeline()
{
	EditAccessCheck();
	return *InterruptTimeline;
}

} //namespace RenderTraceInsights
} //namespace UE
