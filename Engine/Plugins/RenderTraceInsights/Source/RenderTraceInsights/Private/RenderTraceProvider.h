// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRenderTraceProvider.h"
#include "InsightsCore/Common/SimpleRtti.h"
#include "Trace/Analyzer.h"
#include "Common/ProviderLock.h"

namespace TraceServices { class IAnalysisSession; }

namespace UE
{
namespace RenderTraceInsights
{

extern thread_local TraceServices::FProviderLock::FThreadLocalState GRenderTraceProviderLockState;

class FRenderTraceProvider final : public IEditableRenderTraceProvider
{
public:
	UE_NONCOPYABLE(FRenderTraceProvider)

	static FName ProviderName;

	FRenderTraceProvider(TraceServices::IAnalysisSession& InSession);

	// IProvider methods.
	void BeginRead() const override { Lock.BeginRead(GRenderTraceProviderLockState); }
	void EndRead() const override { Lock.EndRead(GRenderTraceProviderLockState); }
	void ReadAccessCheck() const override { Lock.ReadAccessCheck(GRenderTraceProviderLockState); }

	// IEditableProvider methods.
	void BeginEdit() const override { Lock.BeginWrite(GRenderTraceProviderLockState); }
	void EndEdit() const override { Lock.EndWrite(GRenderTraceProviderLockState); }
	void EditAccessCheck() const override { Lock.WriteAccessCheck(GRenderTraceProviderLockState); }

	/////////////////////////////////////////
	// IRenderTraceProvider methods.

	uint64 GetTimelinesModCount() const override { return TimelinesModCount; }

	int32 GetNumCommandLists() const override;
	const FCommandListInstance& GetCommandList(uint32 ID) const override;
	int32 GetNumCommandListTimelines() const override;
	void ReadCommandListTimelines(TFunctionRef<void(uint32, const TEventTimeline&)> Callback) const override;

	int32 GetNumRDGPasses() const override;
	const FRDGPassInstance& GetRDGPass(uint32 ID) const override;
	void ReadRDGTimelines(TFunctionRef<void(uint32, uint32, const TEventTimeline&)> Callback) const override;
	const TEventTimeline* GetRenderThreadSubmissionTimeline() const override;

	int32 GetNumRHITranslateTasks() const override;
	const FRHITranslateTask& GetRHITranslateTask(uint32 ID) const override;
	uint32 FindRHITaskByPredicate(uint32 StartAtTaskID, double TimeRange, TFunctionRef<int(const FRHITranslateTask&)> Pred) const override;
	void EnumerateRHITranslateTimelines(TFunctionRef<void(uint32, uint32, const TEventTimeline&)> Callback) const override;
	const TEventTimeline* GetRHISubmissionTimeline() const override;

	int32 GetNumPlatformPayloads() const override;
	const FPlatformPayload& GetPlatformPayload(uint32 ID) const override;

	int32 GetNumSyncPoints() const override;
	const FSyncPoint& GetSyncPoint(uint32 ID) const override;

	int32 GetNumSubmissionBatches() const override;
	const FSubmissionBatch& GetSubmissionBatch(uint32 ID) const override;
	const TEventTimeline* GetSubmissionQueueTimeline() const override;

	int32 GetNumInterruptWakeUps() const override;
	const FInterruptWakeUp& GetInterruptWakeUp(uint32 ID) const override;
	const TEventTimeline* GetInterruptTimeline() const override;

	/////////////////////////////////////////
	// IEditableRenderTraceProvider methods.

	TPair<uint32, FCommandListInstance&> AddCommandList(uint64 AppID, double Timestamp, ECommandListType Type) override;
	FCommandListInstance& EditCommandList(uint32 ID) override;
	void EnumerateCommandListsForEdit(TFunctionRef<void(uint32, FCommandListInstance&)> Callback) override;
	TPair<uint32, TEventTimeline&> AddCommandListTimeline() override;
	TEventTimeline& EditCommandListTimeline(uint32 Index) override;

	TPair<uint32, FRDGPassInstance&> AddRDGPass(const TCHAR* Name, uint32 ThreadID, double Timestamp, ERDGPassType Type) override;
	FRDGPassInstance& EditRDGPass(uint32 ID) override;
	TEventTimeline& EditRDGThreadTimeline(uint32 ThreadID) override;
	TEventTimeline& EditRenderThreadSubmissionTimeline() override;

	TPair<uint32, FRHITranslateTask&> AddRHITranslateTask(uint64 AppID, ERHITranslateTaskType Type, uint32 ThreadID, double Timestamp) override;
	FRHITranslateTask& EditRHITranslateTask(uint32 ID) override;
	TEventTimeline& EditRHITranslateThreadTimeline(uint32 ThreadID) override;
	TEventTimeline& EditRHISubmissionTimeline() override;

	TPair<uint32, FPlatformPayload&> AddPlatformPayload(uint64 InAppID, double InStartTime, uint8 InPipeIdx) override;
	FPlatformPayload& EditPlatformPayload(uint32 ID) override;

	TPair<uint32, FSyncPoint&> AddSyncPoint(uint64 InAppID, ESyncPointType InType) override;
	FSyncPoint& EditSyncPoint(uint32 ID) override;

	TPair<uint32, FSubmissionBatch&> AddSubmissionBatch(uint32 ThreadID, double Timestamp) override;
	FSubmissionBatch& EditSubmissionBatch(uint32 ID) override;
	TEventTimeline& EditSubmissionQueueTimeline() override;

	TPair<uint32, FInterruptWakeUp&> AddInterruptWakeUp(uint32 ThreadID, double Timestamp) override;
	FInterruptWakeUp& EditInterruptWakeUp(uint32 ID) override;
	TEventTimeline& EditInterruptTimeline() override;

private:
	struct FThreadTimeline
	{
		FThreadTimeline(uint32 InThreadID, TraceServices::ILinearAllocator& InAllocator)
			: ThreadID(InThreadID)
			, Timeline(MakeUnique<TEventTimeline>(InAllocator))
		{}

		uint32 ThreadID;
		TUniquePtr<TEventTimeline> Timeline;
	};

	TraceServices::IAnalysisSession& Session;

	TArray<FCommandListInstance> CommandLists;
	TArray<TUniquePtr<TEventTimeline>> CommandListTimelines;

	TArray<FRDGPassInstance> RDGPasses;
	TArray<FThreadTimeline> RDGTimelines;
	TMap<uint32, uint32> RDGThreadToTimeline;
	TUniquePtr<TEventTimeline> RTSubmissionTimeline;

	TArray<FRHITranslateTask> RHITranslateTasks;
	TArray<FThreadTimeline> RHITranslateTimelines;
	TMap<uint32, uint32> RHIThreadToTimeline;
	TUniquePtr<TEventTimeline> RHISubmissionTimeline;

	TArray<FPlatformPayload> PlatformPayloads;
	TArray<FSyncPoint> SyncPoints;

	TArray<FSubmissionBatch> SubmissionBatches;
	TUniquePtr<TEventTimeline> SubmissionQueueTimeline;

	TArray<FInterruptWakeUp> InterruptWakeUps;
	TUniquePtr<TEventTimeline> InterruptTimeline;

	mutable TraceServices::FProviderLock Lock;

	// This is incremented whenever a timeline is added to let FRenderTraceTimingViewSession know that it needs to add
	// the corresponding tracks. We don't track other modifications right now, such as adding command lists, RDG passes etc.
	// We start at 1 because FRenderTraceTimingViewSession starts its cached value at 0, so this was we always trigger an
	// update on the first tick.
	uint64 TimelinesModCount = 1;
};

} //namespace RenderTraceInsights
} //namespace UE
