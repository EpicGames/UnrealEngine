// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderTraceTimingViewExtender.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/IUnrealInsightsModule.h"

#define LOCTEXT_NAMESPACE "RenderTraceTimingViewExtender"

namespace UE
{
namespace RenderTraceInsights
{

void FRenderTraceTimingViewExtender::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	if (InSession.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
	{
		return;
	}

	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if(PerSessionData == nullptr)
	{
		PerSessionData = &PerSessionDataMap.Add(&InSession);
		PerSessionData->SharedData = MakeShared<FRenderTraceTimingViewSession>();

		PerSessionData->SharedData->OnBeginSession(InSession);
	}
	else
	{
		PerSessionData->SharedData->OnBeginSession(InSession);
	}
}

void FRenderTraceTimingViewExtender::OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	if (InSession.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
	{
		return;
	}

	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if(PerSessionData != nullptr)
	{
		PerSessionData->SharedData->OnEndSession(InSession);
	}

	PerSessionDataMap.Remove(&InSession);
}

void FRenderTraceTimingViewExtender::Tick(UE::Insights::Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if(PerSessionData != nullptr)
	{
		PerSessionData->SharedData->Tick(InSession, InAnalysisSession);
	}
}

void FRenderTraceTimingViewExtender::ExtendFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder)
{
	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if (PerSessionData != nullptr)
	{
		PerSessionData->SharedData->ExtendFilterMenu(InMenuBuilder);
	}
}

} //namespace RenderTraceInsights
} //namespace UE

#undef LOCTEXT_NAMESPACE
