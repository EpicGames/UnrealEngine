// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/ITimingViewExtender.h"
#include "RenderTraceTimingViewSession.h"

namespace UE::Insights::Timing { class ITimingViewSession; }
namespace TraceServices { class IAnalysisSession; }
class FMenuBuilder;

namespace UE
{
namespace RenderTraceInsights
{

class FRenderTraceTimingViewExtender : public UE::Insights::Timing::ITimingViewExtender
{
public:
	//~ Begin UE::Insights::Timing::ITimingViewExtender interface
	virtual void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
	virtual void OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
	virtual void Tick(UE::Insights::Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession) override;
	virtual void ExtendFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder) override;
	//~ End UE::Insights::Timing::ITimingViewExtender interface

private:
	struct FPerSessionData
	{
		TSharedPtr<FRenderTraceTimingViewSession> SharedData;
	};

	// The data we host per-session
	TMap<UE::Insights::Timing::ITimingViewSession*, FPerSessionData> PerSessionDataMap;
};

} //namespace RenderTraceInsights
} //namespace UE
