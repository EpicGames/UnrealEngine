// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderTraceTimingViewSession.h"
#include "IRenderTraceProvider.h"
#include "RenderTraceTrack.h"
#include "Insights/ITimingViewSession.h"
#include "TraceServices/Model/Threads.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/TimingProfiler/Tracks/ThreadTimingTrack.h"
#include "Common/ProviderLock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "RenderTraceTimingViewSession"

namespace UE
{
namespace RenderTraceInsights
{

void FRenderTraceTimingViewSession::ClearState()
{
	RDGPassTracks.Empty();
	RDGThreadToTrackIndex.Empty();

	CommandListTracks.Empty();

	RHIThreadTracks.Empty();
	RHIThreadToTrackIndex.Empty();

	SubmissionQueueTrack.Reset();
	InterruptTrack.Reset();

	CpuThreadTrackCache.Empty();

	TimingViewSession->OnSelectedEventChanged().RemoveAll(this);

	AnalysisSession = nullptr;
	RenderTraceProvider = nullptr;
	LastTimelinesModCount = 0;
}

void FRenderTraceTimingViewSession::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession)
{
	TimingViewSession = &InTimingViewSession;

	ClearState();

	TimingViewSession->OnSelectedEventChanged().AddSP(SharedThis(this), &FRenderTraceTimingViewSession::OnEventSelected);
}

void FRenderTraceTimingViewSession::OnEndSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession)
{
	TimingViewSession = &InTimingViewSession;

	ClearState();

	TimingViewSession = nullptr;
}

void FRenderTraceTimingViewSession::Tick(UE::Insights::Timing::ITimingViewSession& InTimingViewSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	TimingViewSession = &InTimingViewSession;
	AnalysisSession = &InAnalysisSession;

	RenderTraceProvider = ReadRenderTraceProvider(*AnalysisSession);
	if (!RenderTraceProvider)
	{
		return;
	}

	TraceServices::FProviderReadScopeLock ProviderReadScope(*RenderTraceProvider);
	if (RenderTraceProvider->GetTimelinesModCount() == LastTimelinesModCount)
	{
		// No changes since last time we were called, nothing to do.
		return;
	}

	LastTimelinesModCount = RenderTraceProvider->GetTimelinesModCount();
	
	TraceServices::FAnalysisSessionReadScope SessionReadScope(GetAnalysisSession());

	const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(*AnalysisSession);
	bool bTrackAdded = false;

	static constexpr int32 RDGTrackOrderStart = FTimingTrackOrder::Last + FTimingTrackOrder::GroupRange;
	static constexpr int32 CmdListTrackOrderStart = RDGTrackOrderStart + 1024;
	static constexpr int32 RHIThreadTrackOrderStart = CmdListTrackOrderStart + 1024;
	static constexpr int32 SubmissionQueueTrackOrderStart = RHIThreadTrackOrderStart + 1024;
	static constexpr int32 InterruptTrackOrderStart = SubmissionQueueTrackOrderStart + 1024;

	RenderTraceProvider->ReadRDGTimelines([this, &ThreadProvider, &bTrackAdded](uint32 Index, uint32 ThreadID, const IRenderTraceProvider::TEventTimeline& Timeline)
	{
		if (Index >= (uint32)RDGPassTracks.Num())
		{
			RDGPassTracks.SetNum(Index + 1);
		}

		if (RDGPassTracks[Index].IsValid())
		{
			return;
		}

		const int32 TrackOrder = RDGTrackOrderStart + Index;
		const TCHAR* ThreadName = ThreadProvider.GetThreadName(ThreadID);
		const FString TrackName = FString::Printf(TEXT("RDG Pass Exec %s"), ThreadName);

		TSharedPtr<FRDGThreadTrack> Track = MakeShared<FRDGThreadTrack>(*this, TrackName, &Timeline);
		Track->SetVisibilityFlag(bTracksVisible);
		Track->SetOrder(TrackOrder);
		RDGPassTracks[Index] = Track;

		RDGThreadToTrackIndex.Add(ThreadID, Index);

		if (FCString::Strncmp(ThreadName, TEXT("RenderThread"), 12) == 0)
		{
			// Add a render thread submission child track to the render thread track.
			TSharedRef<FRenderThreadSubmissionTrack> SubmissionTrack = MakeShared<FRenderThreadSubmissionTrack>(*this, TEXT("RT Submission"), RenderTraceProvider->GetRenderThreadSubmissionTimeline());
			SubmissionTrack->SetParentTrack(Track);
			Track->AddChildTrack(SubmissionTrack);
		}

		TimingViewSession->AddScrollableTrack(Track);
		bTrackAdded = true;
	});

	RenderTraceProvider->ReadCommandListTimelines([this, &bTrackAdded](uint32 Index, const IRenderTraceProvider::TEventTimeline& Timeline)
	{
		if (Index >= (uint32)CommandListTracks.Num())
		{
			CommandListTracks.SetNum(Index + 1);
		}

		if (CommandListTracks[Index].IsValid())
		{
			return;
		}

		const int32 TrackOrder = CmdListTrackOrderStart + Index;
		const FString TrackName = FString::Printf(TEXT("RHI CmdList Lane %u"), Index);

		TSharedPtr<FCommandListTrack> Track = MakeShared<FCommandListTrack>(*this, TrackName, &Timeline);
		Track->SetVisibilityFlag(bTracksVisible);
		Track->SetOrder(TrackOrder);
		CommandListTracks[Index] = Track;

		TimingViewSession->AddScrollableTrack(Track);
		bTrackAdded = true;
	});

	RenderTraceProvider->EnumerateRHITranslateTimelines([this, &ThreadProvider, &bTrackAdded](uint32 Index, uint32 ThreadID, const IRenderTraceProvider::TEventTimeline& Timeline)
	{
		if (Index >= (uint32)RHIThreadTracks.Num())
		{
			RHIThreadTracks.SetNum(Index + 1);
		}

		if (RHIThreadTracks[Index].IsValid())
		{
			return;
		}

		const int32 TrackOrder = RHIThreadTrackOrderStart + Index;
		const TCHAR* ThreadName = ThreadProvider.GetThreadName(ThreadID);
		const FString TrackName = FString::Printf(TEXT("RHI Tasks %s"), ThreadName);

		TSharedPtr<FRHIThreadTrack> Track = MakeShared<FRHIThreadTrack>(*this, TrackName, &Timeline);
		Track->SetVisibilityFlag(bTracksVisible);
		Track->SetOrder(TrackOrder);
		RHIThreadTracks[Index] = Track;

		RHIThreadToTrackIndex.Add(ThreadID, Index);

		if (FCString::Strncmp(ThreadName, TEXT("RHIThread"), 9) == 0)
		{
			// Add an RHI submission child track to the RHI thread track.
			TSharedRef<FRHISubmissionTrack> SubmissionTrack = MakeShared<FRHISubmissionTrack>(*this, TEXT("RHI Submission"), RenderTraceProvider->GetRHISubmissionTimeline());
			SubmissionTrack->SetParentTrack(Track);
			Track->AddChildTrack(SubmissionTrack);
		}

		TimingViewSession->AddScrollableTrack(Track);
		bTrackAdded = true;
	});

	if (!SubmissionQueueTrack.IsValid())
	{
		const IRenderTraceProvider::TEventTimeline* Timeline = RenderTraceProvider->GetSubmissionQueueTimeline();
		if (Timeline)
		{
			SubmissionQueueTrack = MakeShared<FSubmissionQueueTrack>(*this, TEXT("Submission Queue"), Timeline);
			SubmissionQueueTrack->SetVisibilityFlag(bTracksVisible);
			SubmissionQueueTrack->SetOrder(SubmissionQueueTrackOrderStart);

			TimingViewSession->AddScrollableTrack(SubmissionQueueTrack);
			bTrackAdded = true;
		}
	}

	if (!InterruptTrack.IsValid())
	{
		const IRenderTraceProvider::TEventTimeline* Timeline = RenderTraceProvider->GetInterruptTimeline();
		if (Timeline)
		{
			InterruptTrack = MakeShared<FInterruptTrack>(*this, TEXT("Interrupt"), Timeline);
			InterruptTrack->SetVisibilityFlag(bTracksVisible);
			InterruptTrack->SetOrder(InterruptTrackOrderStart);

			TimingViewSession->AddScrollableTrack(InterruptTrack);
			bTrackAdded = true;
		}
	}

	if (bTrackAdded)
	{
		TimingViewSession->InvalidateScrollableTracksOrder();
	}
}

void FRenderTraceTimingViewSession::ExtendFilterMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("RenderTraceTracks", LOCTEXT("RenderTraceHeader", "RenderTrace"));
	{
		InMenuBuilder.AddMenuEntry(
			LOCTEXT("RenderTraceTimingTracks", "RenderTrace Tracks"),
			LOCTEXT("RenderTraceTimingTracks_Tooltip", "Show/hide the RenderTrace tracks"),
			FSlateIcon(),
			FUIAction(
				// CreateRaw seems to be OK here because of how the UI lifetime is managed. A shared pointer would be
				// safer, but I don't want to introduce a cyclical dependency and this pattern is used in all other
				// view session extenders anyway.
				FExecuteAction::CreateRaw(this, &FRenderTraceTimingViewSession::ToggleRenderTraceTrack),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return bTracksVisible; })
				),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	InMenuBuilder.EndSection();
}

TSharedPtr<const FRDGThreadTrack> FRenderTraceTimingViewSession::GetRDGThreadTrack(uint32 ThreadID) const
{
	const uint32* TrackIndex = RDGThreadToTrackIndex.Find(ThreadID);
	if (!TrackIndex)
	{
		return nullptr;
	}

	return RDGPassTracks[*TrackIndex];
}

TSharedPtr<const FRHIThreadTrack> FRenderTraceTimingViewSession::GetRHIThreadTrack(uint32 ThreadID) const
{
	const uint32* TrackIndex = RHIThreadToTrackIndex.Find(ThreadID);
	if (!TrackIndex)
	{
		return nullptr;
	}

	return RHIThreadTracks[*TrackIndex];
}

void FRenderTraceTimingViewSession::ToggleRenderTraceTrack()
{
	bTracksVisible = !bTracksVisible;

	for (TSharedPtr<FRDGThreadTrack>& Track : RDGPassTracks)
	{
		if (Track.IsValid())
		{
			Track->SetVisibilityFlag(bTracksVisible);
		}
	}

	for (TSharedPtr<FCommandListTrack>& Track : CommandListTracks)
	{
		if (Track.IsValid())
		{
			Track->SetVisibilityFlag(bTracksVisible);
		}
	}

	for (TSharedPtr<FRHIThreadTrack>& Track : RHIThreadTracks)
	{
		if (Track.IsValid())
		{
			Track->SetVisibilityFlag(bTracksVisible);
		}
	}

	if (SubmissionQueueTrack.IsValid())
	{
		SubmissionQueueTrack->SetVisibilityFlag(bTracksVisible);
	}

	if (InterruptTrack.IsValid())
	{
		InterruptTrack->SetVisibilityFlag(bTracksVisible);
	}
}

void FRenderTraceTimingViewSession::OnEventSelected(TSharedPtr<const ITimingEvent> InSelectedEvent)
{
	// This is called by the global OnSelectedEventChanged() event. We need to catch that so we can do global operations
	// such as clearing dependency arrows when the selection is cleared or changed to something that doesn't belong to us.
	// After we've done this we can unpack and route the event to our tracks, so they don't need to use the
	// FBaseTimingTrack::OnEventSelected override.

	if (!TimingViewSession || !RenderTraceProvider)
	{
		return;
	}

	// Lock the provider for reading for convenience, since all the OnRenderTraceEventSelected() implementations will read from it.
	TraceServices::FProviderReadScopeLock ProviderReadScope(*RenderTraceProvider);

	// Clear any dependency arrows that might currently exist.
	TimingViewSession->EditCurrentRelations().RemoveAll([](TUniquePtr<ITimingEventRelation>& Item)
	{
		return Item->Is<FRenderTraceRelation>();
	});

	// RenderTrace tracks use FTimingEvent to represent events, so ignore everything else.
	if (!InSelectedEvent.IsValid() || !InSelectedEvent->Is<FTimingEvent>())
	{
		return;
	}

	// We only care about our own tracks.
	TSharedRef<const FBaseTimingTrack> BaseTrack = InSelectedEvent->GetTrack();
	if (!BaseTrack->Is<FRenderTraceTrack>())
	{
		return;
	}

	// Route the event to the corresponding track.
	const FRenderTraceTrack& Track = BaseTrack->As<const FRenderTraceTrack>();
	const FTimingEvent& Event = InSelectedEvent->As<FTimingEvent>();
	Track.OnRenderTraceEventSelected(Event);
}

TSharedPtr<const FBaseTimingTrack> FRenderTraceTimingViewSession::GetCpuThreadTrack(uint32 ThreadID) const
{
	if (!TimingViewSession)
	{
		return nullptr;
	}

	// Check cache first.
	if (TWeakPtr<FBaseTimingTrack>* Cached = CpuThreadTrackCache.Find(ThreadID))
	{
		TSharedPtr<FBaseTimingTrack> Pinned = Cached->Pin();
		if (Pinned)
		{
			return Pinned;
		}

		// Stale entry, remove it.
		CpuThreadTrackCache.Remove(ThreadID);
	}

	// Cache all the thread timing tracks we can find.
	TSharedPtr<FBaseTimingTrack> FoundTrack;
	TimingViewSession->EnumerateTracks([this, &FoundTrack, ThreadID](TSharedPtr<FBaseTimingTrack> Track)
	{
		if (Track->Is<UE::Insights::TimingProfiler::FThreadTimingTrack>())
		{
			UE::Insights::TimingProfiler::FThreadTimingTrack& ThreadTrack = Track->As<UE::Insights::TimingProfiler::FThreadTimingTrack>();
			CpuThreadTrackCache.Add(ThreadTrack.GetThreadId(), Track);
			if (ThreadTrack.GetThreadId() == ThreadID)
			{
				FoundTrack = Track;
			}
		}
	});

	return FoundTrack;
}

} //namespace RenderTraceInsights
} //namespace UE

#undef LOCTEXT_NAMESPACE
