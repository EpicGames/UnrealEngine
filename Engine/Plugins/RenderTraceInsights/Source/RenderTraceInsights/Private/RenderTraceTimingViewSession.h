// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

namespace TraceServices { class IAnalysisSession; }
namespace UE::Insights::Timing { class ITimingViewSession; }

class ITimingEvent;
class FMenuBuilder;
class FBaseTimingTrack;

namespace UE
{
namespace RenderTraceInsights
{

class IRenderTraceProvider;
class FRenderTraceTrack;
class FCommandListTrack;
class FRDGThreadTrack;
class FRHIThreadTrack;
class FSubmissionQueueTrack;
class FInterruptTrack;

class FRenderTraceTimingViewSession : public TSharedFromThis<FRenderTraceTimingViewSession>
{
public:
	FRenderTraceTimingViewSession() = default;

	void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession);
	void OnEndSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession);
	void Tick(UE::Insights::Timing::ITimingViewSession& InTimingViewSession, const TraceServices::IAnalysisSession& InAnalysisSession);
	void ExtendFilterMenu(FMenuBuilder& InMenuBuilder);

	const TraceServices::IAnalysisSession& GetAnalysisSession() const { return *AnalysisSession; }
	UE::Insights::Timing::ITimingViewSession* GetTimingViewSession() const { return TimingViewSession; }
	const IRenderTraceProvider* GetRenderTraceProvider() const { return RenderTraceProvider; }
	TSharedPtr<const FRDGThreadTrack> GetRDGThreadTrack(uint32 ThreadID) const;
	TSharedPtr<const FCommandListTrack> GetCommandListTrack(int32 Index) const { return Index >= 0 && Index < CommandListTracks.Num() ? CommandListTracks[Index] : nullptr; }
	TSharedPtr<const FRHIThreadTrack> GetRHIThreadTrack(uint32 ThreadID) const;
	TSharedPtr<const FSubmissionQueueTrack> GetSubmissionTrack() const { return SubmissionQueueTrack; }
	TSharedPtr<const FInterruptTrack> GetInterruptTrack() const { return InterruptTrack; }

	TSharedPtr<const FBaseTimingTrack> GetCpuThreadTrack(uint32 ThreadID) const;

private:
	void ClearState();
	void ToggleRenderTraceTrack();
	void OnEventSelected(TSharedPtr<const ITimingEvent> InSelectedEvent);

	const TraceServices::IAnalysisSession* AnalysisSession = nullptr;
	UE::Insights::Timing::ITimingViewSession* TimingViewSession = nullptr;
	const IRenderTraceProvider* RenderTraceProvider = nullptr;
	
	TArray<TSharedPtr<FRDGThreadTrack>> RDGPassTracks;
	TMap<uint32, uint32> RDGThreadToTrackIndex;

	TArray<TSharedPtr<FCommandListTrack>> CommandListTracks;

	TArray<TSharedPtr<FRHIThreadTrack>> RHIThreadTracks;
	TMap<uint32, uint32> RHIThreadToTrackIndex;

	TSharedPtr<FSubmissionQueueTrack> SubmissionQueueTrack;
	TSharedPtr<FInterruptTrack> InterruptTrack;

	bool bTracksVisible = true;
	uint64 LastTimelinesModCount = 0;

	mutable TMap<uint32, TWeakPtr<FBaseTimingTrack>> CpuThreadTrackCache;
};

} //namespace RenderTraceInsights
} //namespace UE
