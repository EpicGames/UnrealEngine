// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderTraceTrack.h"
#include "RenderTraceProvider.h"
#include "RenderTraceTimingViewSession.h"
#include "RenderTraceModule.h"
#include "InsightsCore/Common/TimeUtils.h"
#include "InsightsCore/Common/PaintUtils.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/ITimingViewDrawHelper.h"
#include "Misc/StringBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Common/ProviderLock.h"

namespace UE {
namespace RenderTraceInsights {

INSIGHTS_IMPLEMENT_RTTI(FRenderTraceRelation)
INSIGHTS_IMPLEMENT_RTTI(FRenderTraceTrack)
INSIGHTS_IMPLEMENT_RTTI(FCommandListTrack)
INSIGHTS_IMPLEMENT_RTTI(FRDGThreadTrack)
INSIGHTS_IMPLEMENT_RTTI(FRenderThreadSubmissionTrack)
INSIGHTS_IMPLEMENT_RTTI(FRHIThreadTrack)
INSIGHTS_IMPLEMENT_RTTI(FRHISubmissionTrack)
INSIGHTS_IMPLEMENT_RTTI(FSubmissionQueueTrack)
INSIGHTS_IMPLEMENT_RTTI(FInterruptTrack)

constexpr FLinearColor GRelationColorRecording(0.0f, 1.0f, 0.0f, 1.0f);
constexpr FLinearColor GRelationColorDetach(1.0f, 1.0f, 0.0f, 1.0f);
constexpr FLinearColor GRelationColorSubmitCmdList(1.0f, 0.647f, 0.0f, 1.0f);
constexpr FLinearColor GRelationColorTranslate(1.0f, 0.0f, 0.0f, 1.0f);
constexpr FLinearColor GRelationColorSplitFromTranslateJob(0.0f, 0.0f, 1.0f, 1.0f);
constexpr FLinearColor GRelationColorSplitToTranslateJob(0.0f, 0.2f, 1.0f, 1.0f);
constexpr FLinearColor GRelationColorTriggerSubmission(1.0f, 0.58f, 0.12f, 1.0f);
constexpr FLinearColor GRelationColorSubmitTranslateJob(1.0f, 0.0f, 1.0f, 1.0f);
constexpr FLinearColor GRelationColorExecute(1.0f, 0.5f, 0.5f, 1.0f);
constexpr FLinearColor GRelationColorResolveSyncPoint(0.0f, 0.5f, 0.5f, 1.0f);
constexpr FLinearColor GRelationColorInterrupt(1.0f, 0.5f, 0.0f, 1.0f);
constexpr FLinearColor GRelationColorExecThread(1.0f, 1.0f, 1.0f, 1.0f);

static const TCHAR* GetRDGPassTypeName(ERDGPassType Type)
{
	switch (Type)
	{
	case ERDGPassType::Regular: return TEXT("Regular");
	case ERDGPassType::Dispatch: return TEXT("Dispatch");
	case ERDGPassType::Submit: break; // This function should never be called for submit passes.
	}

	checkNoEntry();
	return TEXT("INVALID");
}

static const TCHAR* GetCommandListTypeName(ECommandListType Type)
{
	switch (Type)
	{
	case ECommandListType::Regular: return TEXT("Regular");
	case ECommandListType::Immediate: return TEXT("Immediate");
	case ECommandListType::Detached: return TEXT("Detached");
	}

	checkNoEntry();
	return TEXT("INVALID");
}

static const TCHAR* GetPipeName(uint32 PipeIdx)
{
	switch (PipeIdx)
	{
	case 0: return TEXT("Graphics");
	case 1: return TEXT("Async Compute");
	case 2: return TEXT("Copy");
	}

	checkNoEntry();
	return TEXT("INVALID");
}

static const TCHAR* GetSyncPointTypeName(ESyncPointType Type)
{
	switch (Type)
	{
	case ESyncPointType::GPU: return TEXT("GPU");
	case ESyncPointType::GPUAndCPU: return TEXT("GPUAndCPU");
	case ESyncPointType::Manual: return TEXT("Manual");
	}

	checkNoEntry();
	return TEXT("INVALID");
}

static FString GetCommandListName(const FCommandListInstance& CmdList)
{
	return FString::Printf(TEXT("0x%llx"), CmdList.AppID);
}

static FString GetTranslateTaskName(const FRHITranslateTask& Task)
{
	return FString::Printf(TEXT("0x%llx"), Task.AppID);
}

static FString GetExitStatusStr(uint8 ExitStatus)
{
	FString ExitStatusStr;
	if (ExitStatus & 0x01) ExitStatusStr += TEXT("Processed ");
	if (ExitStatus & 0x02) ExitStatusStr += TEXT("Pending ");
	if (ExitStatusStr.IsEmpty()) ExitStatusStr = TEXT("Empty");
	return ExitStatusStr;
}

static TSharedPtr<const FBaseTimingTrack> GetFirstChildTrack(TSharedPtr<const FTimingEventsTrack> Track)
{
	if (!Track.IsValid())
	{
		return nullptr;
	}

	TArrayView<const TSharedRef<FBaseTimingTrack>> Children = Track->GetChildTracks();
	if (Children.IsEmpty())
	{
		return nullptr;
	}

	return Children[0];
}

#define ADD_FLAG_TO_LIST(Field, FlagNamespace, FlagName) if (Field & FlagNamespace##_##FlagName) FlagList += TEXT(#FlagName " ")

constexpr uint32 RELATED_TRANSLATE_JOB_NOT_FOUND = INVALID_EVENT_ID - 1;

enum class EEventDetailLevel
{
	Brief,
	Full
};

class IEventDetailsBuilder
{
public:
	explicit IEventDetailsBuilder(const IRenderTraceProvider& InProvider) : Provider(InProvider) {}
	virtual ~IEventDetailsBuilder() {}
	virtual EEventDetailLevel GetDetailLevel() const = 0;
	virtual void AddTitle(FStringView Title) = 0;
	virtual void AddNameValueTextLine(FStringView Name, FStringView Value) = 0;
	const IRenderTraceProvider& GetProvider() const { return Provider; }
protected:
	const IRenderTraceProvider& Provider;
};

class FTooltipEventDetailsBuilder : public IEventDetailsBuilder
{
public:
	FTooltipEventDetailsBuilder(const IRenderTraceProvider& InProvider, FTooltipDrawState& InTooltip) 
		: IEventDetailsBuilder(InProvider)
		, Tooltip(InTooltip)
	{
		Tooltip.ResetContent();
	}

	~FTooltipEventDetailsBuilder() {}

	EEventDetailLevel GetDetailLevel() const override { return EEventDetailLevel::Brief; }

	void AddTitle(FStringView Title) override
	{
		Tooltip.AddTitle(Title);
	}

	void AddNameValueTextLine(FStringView Name, FStringView Value) override
	{
		Tooltip.AddNameValueTextLine(Name, Value);
	}

	void Finish()
	{
		Tooltip.UpdateLayout();
	}

private:
	FTooltipDrawState& Tooltip;
};

class FClipboardEventDetailsBuilder : public IEventDetailsBuilder
{
public:
	FClipboardEventDetailsBuilder(const IRenderTraceProvider& InProvider) : IEventDetailsBuilder(InProvider) {}
	~FClipboardEventDetailsBuilder() {}

	EEventDetailLevel GetDetailLevel() const override { return EEventDetailLevel::Full; }

	void AddTitle(FStringView Title) override
	{
		StringBuilder << Title;
		StringBuilder << TEXT("\n");
	}

	void AddNameValueTextLine(FStringView Name, FStringView Value) override
	{
		StringBuilder << Name;
		StringBuilder << TEXT(" ");
		StringBuilder << Value;
		StringBuilder << TEXT("\n");
	}

	void Finish()
	{
		FPlatformApplicationMisc::ClipboardCopy(StringBuilder.ToString());
	}

private:
	TStringBuilder<10240> StringBuilder;
};

template<typename ItemType> uint64 GetAppIDForItemID(const IRenderTraceProvider& Provider, uint32 ItemID);
template<> uint64 GetAppIDForItemID<FCommandListInstance>(const IRenderTraceProvider& Provider, uint32 ItemID) { return Provider.GetCommandList(ItemID).AppID; }
template<> uint64 GetAppIDForItemID<FRHITranslateTask>(const IRenderTraceProvider& Provider, uint32 ItemID) { return Provider.GetRHITranslateTask(ItemID).AppID; }

template<typename ItemType>
class FAppIDListBuilder
{
public:
	explicit FAppIDListBuilder(IEventDetailsBuilder& InDetailsBuilder, int32 InNumItems)
		: DetailsBuilder(InDetailsBuilder)
		, NumItems(InNumItems)
	{}
	virtual ~FAppIDListBuilder() {}

	void AddItem(uint64 AppID)
	{
		Line += FString::Printf(TEXT("0x%llx"), AppID);
		if (ItemIdx < NumItems - 1)
		{
			Line += TEXT(", ");
		}
		++ItemIdx;

		if ((ItemIdx % 8) == 0)
		{
			DetailsBuilder.AddNameValueTextLine(TEXTVIEW(""), Line);
			Line = TEXT("");
		}
	}
	
	void AddEvents(TConstArrayView<FEventInterval> Events, bool bFinish)
	{
		for (const FEventInterval& Event : Events)
		{
			uint64 AppID = GetAppIDForItemID<ItemType>(DetailsBuilder.GetProvider(), Event.ItemID);
			AddItem(AppID);
		}

		if (bFinish)
		{
			Finish();
		}
	}

	void AddIDs(TConstArrayView<uint64> AppIDs, bool bFinish)
	{
		for (uint64 ID : AppIDs)
		{
			AddItem(ID);
		}

		if (bFinish)
		{
			Finish();
		}
	}

	void Finish()
	{
		if (!Line.IsEmpty())
		{
			DetailsBuilder.AddNameValueTextLine(TEXTVIEW(""), Line);
		}
	}

	static void AddAllEvents(IEventDetailsBuilder& DetailsBuilder, const TCHAR* Title, TConstArrayView<FEventInterval> Events)
	{
		FAppIDListBuilder<ItemType> ListBuilder(DetailsBuilder, Events.Num());
		DetailsBuilder.AddNameValueTextLine(Title, LexToString(Events.Num()));
		ListBuilder.AddEvents(Events, true);
	}

protected:
	IEventDetailsBuilder& DetailsBuilder;
	int32 NumItems;
	int32 ItemIdx = 0;
	FString Line;
};

FRenderTraceRelation::FRenderTraceRelation(TSharedPtr<const FBaseTimingTrack> InSourceTrack, double InSourceTime, uint32 InSourceDepth, TSharedPtr<const FBaseTimingTrack> InTargetTrack, double InTargetTime, uint32 InTargetDepth, const FLinearColor& InColor)
	: SourceTrack(InSourceTrack)
	, TargetTrack(InTargetTrack)
	, SourceTime(InSourceTime)
	, TargetTime(InTargetTime)
	, SourceDepth(InSourceDepth)
	, TargetDepth(InTargetDepth)
	, Color(InColor)
{
}

void FRenderTraceRelation::Draw(const UE::Insights::FDrawContext& DrawContext, const FTimingTrackViewport& Viewport, const ITimingViewDrawHelper& Helper, const ITimingEventRelation::EDrawFilter Filter)
{
	int32 LayerId = Helper.GetRelationLayerId();

	TSharedPtr<const FBaseTimingTrack> SourceTrackShared = SourceTrack.Pin();
	TSharedPtr<const FBaseTimingTrack> TargetTrackShared = TargetTrack.Pin();

	if (!SourceTrackShared.IsValid() || !TargetTrackShared.IsValid() || (!SourceTrackShared->IsVisible() && !TargetTrackShared->IsVisible()))
	{
		return;
	}

	if (Filter == ITimingEventRelation::EDrawFilter::BetweenScrollableTracks)
	{
		if (SourceTrackShared->GetLocation() != ETimingTrackLocation::Scrollable ||
			TargetTrackShared->GetLocation() != ETimingTrackLocation::Scrollable)
		{
			return;
		}
	}

	if (Filter == ITimingEventRelation::EDrawFilter::BetweenDockedTracks)
	{
		if (SourceTrackShared->GetLocation() == ETimingTrackLocation::Scrollable &&
			TargetTrackShared->GetLocation() == ETimingTrackLocation::Scrollable)
		{
			return;
		}

		LayerId = DrawContext.LayerId;
	}

	FVector2D StartPoint, EndPoint;

	StartPoint.X = Viewport.TimeToSlateUnitsRounded(SourceTime);
	EndPoint.X = Viewport.TimeToSlateUnitsRounded(TargetTime);
	if (FMath::Max(StartPoint.X, EndPoint.X) < 0.0f || FMath::Min(StartPoint.X, EndPoint.X) > Viewport.GetWidth())
	{
		return;
	}

	const auto ComputeY = [&Viewport](const FBaseTimingTrack* Track, uint32 Depth) -> float {
		const float ChildTracksHeight = Track->GetChildTracksTopHeight(Viewport.GetLayout());
		const float LaneY = Viewport.GetLayout().GetLaneY(Depth) + Viewport.GetLayout().EventH / 2.0f;
		return Track->GetPosY() + ChildTracksHeight + LaneY;
	};

	StartPoint.Y = ComputeY(SourceTrackShared.Get(), SourceDepth);
	EndPoint.Y = ComputeY(TargetTrackShared.Get(), TargetDepth);

	const double Distance = FVector2D::Distance(StartPoint, EndPoint);
	constexpr double LineHeightAtStart = 4.0;
	constexpr double LineLengthAtStart = 4.0;
	constexpr double LineLengthAtEnd = 12.0;
	constexpr float OutlineThickness = 5.0f;
	constexpr float LineThickness = 3.0f;
	constexpr double ArrowDirectionLen = 10.0;
	constexpr double ArrowRotationAngle = 20.0;

	const FVector2D StartDir(FMath::Max((EndPoint.X - StartPoint.X), 4.0 * (LineLengthAtStart + LineLengthAtEnd)), 0.0);
	FVector2D ArrowDirection;

	// This makes a heap allocation for only two points, but sadly FSlateDrawElement::MakeLines wants a TArray so we can't
	// even add an inline allocator. That API needs to be changed to take a TArrayView.
	TArray<FVector2D> LinePoints;
	LinePoints.SetNum(2);

	constexpr FLinearColor OutlineColor(0.0f, 0.0f, 0.0f, 1.0f);
	const int32 OutlineLayerId = LayerId - 1;

	const auto DrawLine = [&DrawContext, LayerId, OutlineLayerId, OutlineColor, Color=Color, &LinePoints](FVector2D Start, FVector2D End) {
		LinePoints[0] = Start;
		LinePoints[1] = End;
		DrawContext.DrawLines(OutlineLayerId, 0.0f, 0.0f, LinePoints, OutlineColor, /*bAntialias=*/ true, OutlineThickness);
		DrawContext.DrawLines(LayerId, 0.0f, 0.0f, LinePoints, Color, /*bAntialias=*/ true, LineThickness);
	};

	// Cap the start of the line.
	DrawLine(StartPoint + FVector2D(0.0, -LineHeightAtStart / 2.0), StartPoint + FVector2D(0.0, LineHeightAtStart / 2.0));

	constexpr double MinDistance = 1.5 * (LineLengthAtStart + LineLengthAtEnd);
	constexpr double MaxDistance = 10000.0; // arbitrary limit to avoid stack overflow in recursive FLineBuilder::Subdivide when rendering splines
	if (Distance > MinDistance && Distance < MaxDistance && !FMath::IsNearlyEqual(StartPoint.Y, EndPoint.Y))
	{
		// Curve with straight start and end sections.
		FVector2D SplineStart(StartPoint.X + LineLengthAtStart, StartPoint.Y);
		FVector2D SplineEnd(EndPoint.X - LineLengthAtEnd, EndPoint.Y);
		DrawLine(StartPoint, SplineStart);

		DrawContext.DrawSpline(OutlineLayerId, 0.0f, 0.0f, SplineStart, StartDir, SplineEnd, StartDir, OutlineThickness, OutlineColor);
		DrawContext.DrawSpline(LayerId, 0.0f, 0.0f, SplineStart, StartDir, SplineEnd, StartDir, LineThickness, Color);

		DrawLine(SplineEnd, EndPoint);

		// The arrow head will be horizontal.
		ArrowDirection = FVector2D(-ArrowDirectionLen, 0.0);
	}
	else
	{
		// No room for curve, use a straight line.
		DrawLine(StartPoint, EndPoint);

		// The arrow head direction will be along the line.
		ArrowDirection = StartPoint - EndPoint;
		ArrowDirection.Normalize();
		ArrowDirection *= ArrowDirectionLen;
	}

	// Arrow head.
	DrawLine(EndPoint, EndPoint + ArrowDirection.GetRotated(-ArrowRotationAngle));
	DrawLine(EndPoint, EndPoint + ArrowDirection.GetRotated(+ArrowRotationAngle));
}

FRenderTraceTrack::FRenderTraceTrack(const FRenderTraceTimingViewSession& InSharedData, const FString& Name, const IRenderTraceProvider::TEventTimeline* InTimeline)
	: FTimingEventsTrack(Name)
	, SharedData(InSharedData)
	, Timeline(InTimeline)
{
}

const TSharedPtr<const ITimingEvent> FRenderTraceTrack::GetEvent(double InTime, double SecondsPerPixel, int32 Depth) const
{
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	if (!Timeline || !Provider)
	{
		return nullptr;
	}

	TraceServices::FProviderReadScopeLock ProviderReadScope(*Provider);

	const double Tolerance = 2.0 * SecondsPerPixel;

	// Use the timeline object to find the event at this time and depth.
	TSharedPtr<FTimingEvent> FoundEvent;
	Timeline->EnumerateEvents(InTime - Tolerance, InTime + Tolerance,
	[this, Depth, InTime, &FoundEvent](double TimelineEventStart, double TimelineEventEnd, uint32 TimelineEventDepth, const FRenderTraceEvent& TimelineEvent)
	{
		if (InTime < TimelineEventStart || InTime > TimelineEventEnd)
		{
			return TraceServices::EEventEnumerate::Continue;
		}

		// Allow the subclass to customize the find event logic.
		FoundEvent = GetEventAtTime(InTime, Depth, TimelineEventStart, TimelineEventEnd, TimelineEventDepth, TimelineEvent);
		if (FoundEvent.IsValid())
		{
			return TraceServices::EEventEnumerate::Stop;
		}

		// In most cases we can just use the timeline event at the corresponding depth.
		if (TimelineEventDepth == Depth)
		{
			// We'll just use FTimingEvent since it has a uint64 field (Type) where we can pack our 32-bit item and parent IDs. Hacky but works for now.
			const uint64 Packed = ((uint64)TimelineEvent.ParentItemID << 32) | TimelineEvent.ItemID;
			FoundEvent = MakeShared<FTimingEvent>(SharedThis(this), TimelineEventStart, TimelineEventEnd, Depth, Packed);
			return TraceServices::EEventEnumerate::Stop;
		}

		return TraceServices::EEventEnumerate::Continue;
	});

	return FoundEvent;
}

void FRenderTraceTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	if (!Timeline || !Provider)
	{
		return;
	}

	TraceServices::FProviderReadScopeLock ProviderReadScope(*Provider);

	const FTimingTrackViewport& Viewport = Context.GetViewport();
	const double SecondsPerPixel = 1.0 / Viewport.GetScaleX();

	Timeline->EnumerateEventsDownSampled(Viewport.GetStartTime(), Viewport.GetEndTime(), SecondsPerPixel, [this, &Builder, Provider](double InStartTime, double InEndTime, uint32 InDepth, const FRenderTraceEvent& InEvent)
	{
		// TODO: use string views here to avoid allocations. Currently not possible because some event names are built on the fly.
		FString EventName = GetEventName(Provider, InDepth, InEvent);
		Builder.AddEvent(InStartTime, InEndTime, InDepth, *EventName);
		CustomizeEventDrawState(Builder, InStartTime, InEndTime, InDepth, InEvent);
		return TraceServices::EEventEnumerate::Continue;
	});
}

void FRenderTraceTrack::OnClipboardCopyEvent(const ITimingEvent& InSelectedEvent) const
{
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	if (!Provider)
	{
		return;
	}

	TraceServices::FProviderReadScopeLock ProviderReadScope(*Provider);

	const FTimingEvent& TimingEvent = InSelectedEvent.As<FTimingEvent>();

	FClipboardEventDetailsBuilder Builder(*Provider);
	PopulateEventDetails(Builder, TimingEvent);
	Builder.Finish();
}

void FRenderTraceTrack::InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& TooltipEvent) const
{
	if (!TooltipEvent.Is<FTimingEvent>())
	{
		return;
	}

	if (!TooltipEvent.CheckTrack(this))
	{
		// The event isn't meant for us, but maybe it's meant for one of our child tracks.
		for (const TSharedRef<FBaseTimingTrack>& Track : GetChildTracks())
		{
			Track->InitTooltip(Tooltip, TooltipEvent);
		}
		return;
	}

	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	if (!Provider)
	{
		Tooltip.ResetContent();
		Tooltip.UpdateLayout();
		return;
	}

	TraceServices::FProviderReadScopeLock ProviderReadScope(*Provider);
	const FTimingEvent& TimingEvent = TooltipEvent.As<FTimingEvent>();

	FTooltipEventDetailsBuilder Builder(*Provider, Tooltip);
	PopulateEventDetails(Builder, TimingEvent);
	Builder.Finish();
}

FCommandListTrack::FCommandListTrack(const FRenderTraceTimingViewSession& InSharedData, const FString& Name, const IRenderTraceProvider::TEventTimeline* InTimeline)
	: FRenderTraceTrack(InSharedData, Name, InTimeline)
{
}

void FCommandListTrack::PopulateCommandListEventDetails(IEventDetailsBuilder& Builder, uint32 CmdListID) const
{
	const IRenderTraceProvider& Provider = Builder.GetProvider();
	const FCommandListInstance& CmdList = Provider.GetCommandList(CmdListID);

	Builder.AddTitle(FString::Printf(TEXT("Command list %s"), *GetCommandListName(CmdList)));
	Builder.AddNameValueTextLine(TEXTVIEW("Type:"), GetCommandListTypeName(CmdList.Type));

	{
		FString FlagList;
		ADD_FLAG_TO_LIST(CmdList.RecordingFlags, ECommandListRecordingFlag, UsesLockFence);
		if (FlagList.IsEmpty()) FlagList = TEXT("None");
		Builder.AddNameValueTextLine(TEXTVIEW("Flags:"), FlagList);
	}

	Builder.AddNameValueTextLine(TEXTVIEW("Creation Time:"), Insights::FormatTime(CmdList.CreateTime, UE::Insights::FTimeValue::Microsecond));

	if (CmdList.FinishRecordingTime != 0.0)
	{
		Builder.AddNameValueTextLine(TEXTVIEW("Finish Recording Time:"), Insights::FormatTime(CmdList.FinishRecordingTime, UE::Insights::FTimeValue::Microsecond));
	}

	if (CmdList.SubmitTime != 0.0)
	{
		Builder.AddNameValueTextLine(TEXTVIEW("Submit Time:"), Insights::FormatTime(CmdList.SubmitTime, UE::Insights::FTimeValue::Microsecond));
	}

	Builder.AddNameValueTextLine(TEXTVIEW("Destruction Time:"), Insights::FormatTime(CmdList.DestroyTime, UE::Insights::FTimeValue::Microsecond));

	if (CmdList.DetachedCmdListID != INVALID_EVENT_ID)
	{
		const FCommandListInstance& DetachedCmdList = Provider.GetCommandList(CmdList.DetachedCmdListID);
		Builder.AddNameValueTextLine(TEXTVIEW("Detached To:"), GetCommandListName(DetachedCmdList));
		Builder.AddNameValueTextLine(TEXTVIEW("Detached At:"), Insights::FormatTime(CmdList.DetachTime, UE::Insights::FTimeValue::Microsecond));
	}

	if (CmdList.SourceCmdListID != INVALID_EVENT_ID)
	{
		const FCommandListInstance& SourceCmdList = Provider.GetCommandList(CmdList.SourceCmdListID);
		Builder.AddNameValueTextLine(TEXTVIEW("Detached From:"), GetCommandListName(SourceCmdList));
		Builder.AddNameValueTextLine(TEXTVIEW("Detached At:"), Insights::FormatTime(SourceCmdList.DetachTime, UE::Insights::FTimeValue::Microsecond));
	}

	if (CmdList.DetachedCmdListID == INVALID_EVENT_ID)
	{
		// The source command list for detach actions is never submitted or translated, no point in showing this information.
		if (CmdList.SubmitPassID != INVALID_EVENT_ID)
		{
			Builder.AddNameValueTextLine(TEXTVIEW("Submitted By:"), LexToString(CmdList.SubmitPassID));
		}
		else
		{
			Builder.AddNameValueTextLine(TEXTVIEW("NOT SUBMITTED"), TEXTVIEW(""));
		}

		if (CmdList.TranslateTaskID != INVALID_EVENT_ID)
		{
			const FRHITranslateTask& TranslateTask = Provider.GetRHITranslateTask(CmdList.TranslateTaskID);
			Builder.AddNameValueTextLine(TEXTVIEW("Translated By:"), GetTranslateTaskName(TranslateTask));
		}
		else
		{
			Builder.AddNameValueTextLine(TEXTVIEW("NOT TRANSLATED"), TEXTVIEW(""));
		}
	}

	if(!CmdList.RecordingEvents.IsEmpty())
	{
		Builder.AddNameValueTextLine(TEXTVIEW("Recording Events:"), LexToString(CmdList.RecordingEvents.Num()));
		Builder.AddNameValueTextLine(TEXTVIEW("Recording Interval:"), FString::Printf(TEXT("%s -> %s"),
			*Insights::FormatTime(CmdList.RecordingEvents[0].Start, UE::Insights::FTimeValue::Microsecond),
			*Insights::FormatTime(CmdList.RecordingEvents.Last().End, UE::Insights::FTimeValue::Microsecond)
		));

		if (Builder.GetDetailLevel() == EEventDetailLevel::Full)
		{
			for (const FEventInterval& RecordingEvent : CmdList.RecordingEvents)
			{
				const FRDGPassInstance& Pass = Provider.GetRDGPass(RecordingEvent.ItemID);
				Builder.AddNameValueTextLine(TEXTVIEW(""), Pass.Name);
			}
		}
	}
}

void FCommandListTrack::PopulateRecordingEventDetails(IEventDetailsBuilder& Builder, uint32 PassID, double StartTime, double EndTime) const
{
	const IRenderTraceProvider& Provider = Builder.GetProvider();
	const FRDGPassInstance& Pass = Provider.GetRDGPass(PassID);

	Builder.AddTitle(Pass.Name);
	Builder.AddNameValueTextLine(TEXTVIEW("Type:"), GetRDGPassTypeName(Pass.Type));
	Builder.AddNameValueTextLine(TEXTVIEW("Recording Start:"), Insights::FormatTime(StartTime, UE::Insights::FTimeValue::Microsecond));
	Builder.AddNameValueTextLine(TEXTVIEW("Recording Duration:"), Insights::FormatTimeAuto(EndTime - StartTime));
}

void FCommandListTrack::PopulateEventDetails(IEventDetailsBuilder& Builder, const FTimingEvent& Event) const
{
	const uint32 ItemID = static_cast<uint32>(Event.GetType());
	switch (Event.GetDepth())
	{
	case 0:
	{
		PopulateCommandListEventDetails(Builder, ItemID);
		break;
	}

	case 1:
	{
		PopulateRecordingEventDetails(Builder, ItemID, Event.GetStartTime(), Event.GetEndTime());
		break;
	}

	default:
		checkNoEntry();
		return;
	}
}

void FCommandListTrack::OnCommandListSelected(const FTimingEvent& InSelectedEvent) const
{
	UE::Insights::Timing::ITimingViewSession* TimingView = SharedData.GetTimingViewSession();
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	check(TimingView && Provider);

	const uint32 CmdListID = static_cast<uint32>(InSelectedEvent.GetType());
	const FCommandListInstance& CmdList = Provider->GetCommandList(CmdListID);

	// Add arrows from all the passes which recorded to this command list.
	for (const FEventInterval& RecordingEvent : CmdList.RecordingEvents)
	{
		const FRDGPassInstance& Pass = Provider->GetRDGPass(RecordingEvent.ItemID);
		TSharedPtr<const FRDGThreadTrack> PassTrack = SharedData.GetRDGThreadTrack(Pass.ExecThreadID);
		if (!PassTrack.IsValid())
		{
			continue;
		}

		TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
			PassTrack, Pass.StartTime, 0, SharedThis(this), RecordingEvent.Start, 1, GRelationColorRecording
		);

		TimingView->AddRelation(Relation);
	}

	// Add arrows to all detached command lists.
	if (CmdList.DetachedCmdListID != INVALID_EVENT_ID)
	{
		const FCommandListInstance& DetachedCmdList = Provider->GetCommandList(CmdList.DetachedCmdListID);
		TSharedPtr<const FCommandListTrack> DetachedCmdListTrack = SharedData.GetCommandListTrack(DetachedCmdList.TimelineIndex);
		if (DetachedCmdListTrack.IsValid())
		{
			TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
				SharedThis(this), CmdList.DetachTime, 0, DetachedCmdListTrack, CmdList.DetachTime, 0, GRelationColorDetach
			);

			TimingView->AddRelation(Relation);
		}
	}

	// If we're detached, add an arrow from the source command list.
	if (CmdList.Type == ECommandListType::Detached && CmdList.SourceCmdListID != INVALID_EVENT_ID)
	{
		const FCommandListInstance& SourceCmdList = Provider->GetCommandList(CmdList.SourceCmdListID);
		if (SourceCmdList.DetachedCmdListID == CmdListID)
		{
			TSharedPtr<const FCommandListTrack> SourceCmdListTrack = SharedData.GetCommandListTrack(SourceCmdList.TimelineIndex);
			if (SourceCmdListTrack.IsValid())
			{
				TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
					SourceCmdListTrack, SourceCmdList.DetachTime, 0, SharedThis(this), SourceCmdList.DetachTime, 0, GRelationColorDetach
				);

				TimingView->AddRelation(Relation);
			}
		}
	}

	// Show the event which submitted this command list.
	if (CmdList.SubmitPassID != INVALID_EVENT_ID)
	{
		const FRDGPassInstance& Pass = Provider->GetRDGPass(CmdList.SubmitPassID);
		TSharedPtr<const FBaseTimingTrack> RTSubmissionTrack = GetFirstChildTrack(SharedData.GetRDGThreadTrack(Pass.ExecThreadID));
		if (RTSubmissionTrack.IsValid())
		{
			TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
				RTSubmissionTrack, Pass.StartTime, 0, SharedThis(this), CmdList.TimelineStart, 0, GRelationColorSubmitCmdList
			);

			TimingView->AddRelation(Relation);
		}
	}

	// Show the translation task.
	if (CmdList.TranslateTaskID != INVALID_EVENT_ID)
	{
		const FRHITranslateTask& Task = Provider->GetRHITranslateTask(CmdList.TranslateTaskID);
		TSharedPtr<const FRHIThreadTrack> TaskTrack = SharedData.GetRHIThreadTrack(Task.ThreadID);
		if (TaskTrack.IsValid())
		{
			for (const FEventInterval& CmdListEvent : Task.ProcessedItems)
			{
				if (CmdListEvent.ItemID == CmdListID)
				{
					TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
						SharedThis(this), CmdList.FinishRecordingTime, 0, TaskTrack, CmdListEvent.Start, 1, GRelationColorTranslate
					);

					TimingView->AddRelation(Relation);
					break;
				}
			}
		}
	}
}

void FCommandListTrack::OnRecordingEventSelected(const FTimingEvent& InSelectedEvent) const
{
	UE::Insights::Timing::ITimingViewSession* TimingView = SharedData.GetTimingViewSession();
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	check(TimingView && Provider);

	const uint32 PassID = static_cast<uint32>(InSelectedEvent.GetType());
	const FRDGPassInstance& Pass = Provider->GetRDGPass(PassID);
	TSharedPtr<const FRDGThreadTrack> PassTrack = SharedData.GetRDGThreadTrack(Pass.ExecThreadID);
	if (!PassTrack.IsValid())
	{
		return;
	}

	TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
		PassTrack, Pass.StartTime, 0, SharedThis(this), InSelectedEvent.GetStartTime(), 1, GRelationColorRecording
	);

	TimingView->AddRelation(Relation);
}

void FCommandListTrack::OnRenderTraceEventSelected(const FTimingEvent& InSelectedEvent) const
{
	UE::Insights::Timing::ITimingViewSession* TimingView = SharedData.GetTimingViewSession();
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	if (!TimingView || !Provider)
	{
		return;
	}

	const uint32 Depth = InSelectedEvent.GetDepth();
	switch (Depth)
	{
	case 0:
		OnCommandListSelected(InSelectedEvent);
		break;

	case 1:
		OnRecordingEventSelected(InSelectedEvent);
		break;

	default:
		checkNoEntry();
		return;
	}
}

FString FCommandListTrack::GetEventName(const IRenderTraceProvider* Provider, uint32 InDepth, const FRenderTraceEvent& InEvent) const
{
	switch (InDepth)
	{
	case 0:
	{
		// Command list events.
		const FCommandListInstance& CmdList = Provider->GetCommandList(InEvent.ItemID);
		return GetCommandListName(CmdList);
	}

	case 1:
	{
		// Pass recording events.
		const FRDGPassInstance& Pass = Provider->GetRDGPass(InEvent.ItemID);
		return Pass.Name;
	}

	default:
	{
		checkNoEntry();
		return TEXT("invalid");
	}
	}
}

FRDGThreadTrack::FRDGThreadTrack(const FRenderTraceTimingViewSession& InSharedData, const FString& Name, const IRenderTraceProvider::TEventTimeline* InTimeline)
	: FRenderTraceTrack(InSharedData, Name, InTimeline)
{
}

void FRDGThreadTrack::OnRenderTraceEventSelected(const FTimingEvent& InSelectedEvent) const
{
	UE::Insights::Timing::ITimingViewSession* TimingView = SharedData.GetTimingViewSession();
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	if (!TimingView || !Provider)
	{
		return;
	}

	const uint32 PassID = static_cast<uint32>(InSelectedEvent.GetType());
	const FRDGPassInstance& Pass = Provider->GetRDGPass(PassID);
	check(Pass.Type != ERDGPassType::Submit);

	// Add an arrow to the CPU thread where the pass was executed.
	TSharedPtr<const FBaseTimingTrack> CpuThreadTrack = SharedData.GetCpuThreadTrack(Pass.ExecThreadID);
	if (CpuThreadTrack)
	{
		TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
			SharedThis(this), Pass.StartTime, 0, CpuThreadTrack, Pass.StartTime, 0, GRelationColorExecThread
		);
		TimingView->AddRelation(Relation);
	}

	// Add a relationship from the pass to the recording event on each command list it used.
	for (uint32 CmdListID : Pass.CommandListIDs)
	{
		const FCommandListInstance& CmdList = Provider->GetCommandList(CmdListID);

		// Some command lists might not be on the timeline, skip them. TimelineIndex is stored when the analyzer
		// assigns a command list to a lane.
		if (CmdList.TimelineIndex < 0)
		{
			continue;
		}

		TSharedPtr<const FCommandListTrack> CmdListTrack = SharedData.GetCommandListTrack(CmdList.TimelineIndex);
		if (!CmdListTrack.IsValid())
		{
			continue;
		}

		for (const FEventInterval& RecordEvent : CmdList.RecordingEvents)
		{
			if (RecordEvent.ItemID != PassID)
			{
				continue;
			}

			// The RDG passes are at depth 0 and the recording events are at depth 1.
			TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
				SharedThis(this), Pass.StartTime, 0, CmdListTrack, RecordEvent.Start, 1, GRelationColorRecording
			);

			TimingView->AddRelation(Relation);

			// There can only be one event per command list.
			break;
		}
	}
}

void FRDGThreadTrack::PopulateEventDetails(IEventDetailsBuilder& Builder, const FTimingEvent& Event) const
{
	const IRenderTraceProvider& Provider = Builder.GetProvider();
	const uint32 PassID = static_cast<uint32>(Event.GetType());
	const FRDGPassInstance& Pass = Provider.GetRDGPass(PassID);
	check(Pass.Type != ERDGPassType::Submit);

	Builder.AddTitle(Pass.Name);
	Builder.AddNameValueTextLine(TEXTVIEW("Type:"), GetRDGPassTypeName(Pass.Type));

	FString FlagList;
	bool bShowFlags = false, bShowTaskMode = false, bShowDuration = false;

	switch (Pass.Type)
	{
	case ERDGPassType::Regular:
		bShowFlags = bShowTaskMode = bShowDuration = true;
		ADD_FLAG_TO_LIST(Pass.Flags, ERDGPassFlag, Raster);
		ADD_FLAG_TO_LIST(Pass.Flags, ERDGPassFlag, Compute);
		ADD_FLAG_TO_LIST(Pass.Flags, ERDGPassFlag, AsyncCompute);
		ADD_FLAG_TO_LIST(Pass.Flags, ERDGPassFlag, Copy);
		ADD_FLAG_TO_LIST(Pass.Flags, ERDGPassFlag, NeverCull);
		ADD_FLAG_TO_LIST(Pass.Flags, ERDGPassFlag, SkipRenderPass);
		ADD_FLAG_TO_LIST(Pass.Flags, ERDGPassFlag, NeverMerge);
		ADD_FLAG_TO_LIST(Pass.Flags, ERDGPassFlag, NeverParallel);
		break;

	case ERDGPassType::Dispatch:
		bShowDuration = true;
		break;

	default:
		checkNoEntry();
		return;
	}

	if (bShowFlags)
	{
		if (FlagList.IsEmpty()) FlagList = TEXT("None");
		Builder.AddNameValueTextLine(TEXTVIEW("Flags:"), FlagList);
	}

	if (bShowTaskMode)
	{
		const TCHAR* TaskModeStr;
		switch (Pass.TaskMode)
		{
		case ERDGPassTaskMode::Inline: TaskModeStr = TEXT("Inline"); break;
		case ERDGPassTaskMode::Await: TaskModeStr = TEXT("Await"); break;
		case ERDGPassTaskMode::Async: TaskModeStr = TEXT("Async"); break;
		default: TaskModeStr = TEXT("INVALID"); break;
		}

		Builder.AddNameValueTextLine(TEXTVIEW("Task Mode:"), TaskModeStr);
	}

	if(bShowDuration)
	{
		Builder.AddNameValueTextLine(TEXTVIEW("Duration:"), Insights::FormatTimeAuto(Event.GetDuration()));
	}

	Builder.AddNameValueTextLine(TEXTVIEW("Command Lists:"), LexToString(Pass.CommandListIDs.Num()));
	for (uint32 CmdListID : Pass.CommandListIDs)
	{
		Builder.AddNameValueTextLine(TEXTVIEW(""), GetCommandListName(Provider.GetCommandList(CmdListID)));
	}
}

FString FRDGThreadTrack::GetEventName(const IRenderTraceProvider* Provider, uint32 InDepth, const FRenderTraceEvent& InEvent) const
{
	const FRDGPassInstance& Pass = Provider->GetRDGPass(InEvent.ItemID);
	return Pass.Name;
}

FRenderThreadSubmissionTrack::FRenderThreadSubmissionTrack(const FRenderTraceTimingViewSession& InSharedData, const FString& Name, const IRenderTraceProvider::TEventTimeline* InTimeline)
	: FRenderTraceTrack(InSharedData, Name, InTimeline)
{
}

void FRenderThreadSubmissionTrack::OnRenderTraceEventSelected(const FTimingEvent& InSelectedEvent) const
{
	UE::Insights::Timing::ITimingViewSession* TimingView = SharedData.GetTimingViewSession();
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	if (!TimingView || !Provider)
	{
		return;
	}

	const uint32 PassID = static_cast<uint32>(InSelectedEvent.GetType());
	const FRDGPassInstance& Pass = Provider->GetRDGPass(PassID);
	check(Pass.Type == ERDGPassType::Submit);

	// Add an arrow to the CPU thread where the pass was executed.
	TSharedPtr<const FBaseTimingTrack> CpuThreadTrack = SharedData.GetCpuThreadTrack(Pass.ExecThreadID);
	if (CpuThreadTrack)
	{
		TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
			SharedThis(this), Pass.StartTime, 0, CpuThreadTrack, Pass.StartTime, 0, GRelationColorExecThread
		);
		TimingView->AddRelation(Relation);
	}

	// Draw arrows from the pass to the command lists it submitted.
	for (uint32 CmdListID : Pass.CommandListIDs)
	{
		const FCommandListInstance& CmdList = Provider->GetCommandList(CmdListID);
		if (CmdList.TimelineIndex < 0)
		{
			continue;
		}

		TSharedPtr<const FCommandListTrack> CmdListTrack = SharedData.GetCommandListTrack(CmdList.TimelineIndex);
		if (!CmdListTrack.IsValid())
		{
			continue;
		}

		TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
			SharedThis(this), Pass.StartTime, 0, CmdListTrack, CmdList.TimelineStart, 0, GRelationColorSubmitCmdList
		);

		TimingView->AddRelation(Relation);
	}

	// If we triggered an RHI submit, add an arrow to the RHI submit task.
	if (Pass.SubmitTaskID != INVALID_EVENT_ID)
	{
		const FRHITranslateTask& Task = Provider->GetRHITranslateTask(Pass.SubmitTaskID);
		check(Task.Type == ERHITranslateTaskType::Submit);
		TSharedPtr<const FBaseTimingTrack> RHISubmissionTrack = GetFirstChildTrack(SharedData.GetRHIThreadTrack(Task.ThreadID));
		if (RHISubmissionTrack.IsValid())
		{
			TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
				SharedThis(this), InSelectedEvent.GetEndTime(), 0, RHISubmissionTrack, Task.StartTime, 0, GRelationColorTriggerSubmission
			);

			TimingView->AddRelation(Relation);
		}
	}
}

FString FRenderThreadSubmissionTrack::GetEventName(const IRenderTraceProvider* Provider, uint32 InDepth, const FRenderTraceEvent& InEvent) const
{
	const FRDGPassInstance& Pass = Provider->GetRDGPass(InEvent.ItemID);
	return Pass.Name;
}

void FRenderThreadSubmissionTrack::PopulateEventDetails(IEventDetailsBuilder& Builder, const FTimingEvent& Event) const
{
	const IRenderTraceProvider& Provider = Builder.GetProvider();
	const uint32 PassID = static_cast<uint32>(Event.GetType());
	const FRDGPassInstance& Pass = Provider.GetRDGPass(PassID);
	check(Pass.Type == ERDGPassType::Submit);

	Builder.AddTitle(TEXTVIEW("Submit"));
	Builder.AddNameValueTextLine(TEXTVIEW("ID:"), LexToString(PassID));

	FString FlagList;
	ADD_FLAG_TO_LIST(Pass.Flags, ERDGPassSubmitFlag, SubmitToGPU);
	ADD_FLAG_TO_LIST(Pass.Flags, ERDGPassSubmitFlag, DeleteResources);
	ADD_FLAG_TO_LIST(Pass.Flags, ERDGPassSubmitFlag, FlushRHIThread);
	ADD_FLAG_TO_LIST(Pass.Flags, ERDGPassSubmitFlag, EndFrame);
	ADD_FLAG_TO_LIST(Pass.Flags, ERDGPassSubmitFlag, EnableBypass);
	ADD_FLAG_TO_LIST(Pass.Flags, ERDGPassSubmitFlag, DisableBypass);
	ADD_FLAG_TO_LIST(Pass.Flags, ERDGPassSubmitFlag, EnableDrawEvents);
	ADD_FLAG_TO_LIST(Pass.Flags, ERDGPassSubmitFlag, DisableDrawEvents);
	if (FlagList.IsEmpty()) FlagList = TEXT("None");
	Builder.AddNameValueTextLine(TEXTVIEW("Flags:"), FlagList);

	if (Pass.SubmitTaskID != INVALID_EVENT_ID)
	{
		Builder.AddNameValueTextLine(TEXTVIEW("RHI Submit Task:"), LexToString(Pass.SubmitTaskID));
	}

	Builder.AddNameValueTextLine(TEXTVIEW("Command Lists:"), LexToString(Pass.CommandListIDs.Num()));
	for (uint32 CmdListID : Pass.CommandListIDs)
	{
		Builder.AddNameValueTextLine(TEXTVIEW(""), GetCommandListName(Provider.GetCommandList(CmdListID)));
	}
}

FRHIThreadTrack::FRHIThreadTrack(const FRenderTraceTimingViewSession& InSharedData, const FString& Name, const IRenderTraceProvider::TEventTimeline* InTimeline)
	: FRenderTraceTrack(InSharedData, Name, InTimeline)
{
}

void FRHIThreadTrack::CacheSplitFromTask(const IRenderTraceProvider& Provider, uint32 TaskID) const
{
	// This method will change a field on a task object with only the reader lock held. This is OK to do because it's only ever
	// called from the UI thread, so there won't be concurrent updates to the cached task ID.

	const FRHITranslateTask& Task = Provider.GetRHITranslateTask(TaskID);
	if (Task.SplitFromTranslateTaskID < INVALID_EVENT_ID)
	{
		// Already cached.
		return;
	}

	if (Task.ProcessedItems.IsEmpty())
	{
		// No command lists means no split events.
		Task.SplitFromTranslateTaskID = RELATED_TRANSLATE_JOB_NOT_FOUND;
		return;
	}

	// The split information is stored in the first translated command list.
	const FCommandListInstance& FirstCmdList = Provider.GetCommandList(Task.ProcessedItems[0].ItemID);
	if (FirstCmdList.TranslatePrevJobAppID == 0)
	{
		// No split information.
		Task.SplitFromTranslateTaskID = RELATED_TRANSLATE_JOB_NOT_FOUND;
		return;
	}

	const uint32 PrevTaskID = Provider.FindRHITaskByPredicate(TaskID, 0.05, [&Task, &FirstCmdList](const FRHITranslateTask& OtherTask) -> int
	{
		if (OtherTask.AppID == FirstCmdList.TranslatePrevJobAppID) return 1;
		// If we encounter another task with the same app ID as ourselves it's a reused pointer, so we've moved past our lifetime.
		return (OtherTask.AppID == Task.AppID) ? -1 : 0;
	});

	if (PrevTaskID == INVALID_EVENT_ID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Translate task %llx (permanent ID %x) split from %llx: could not find previous task.", Task.AppID, TaskID, FirstCmdList.TranslatePrevJobAppID);
		Task.SplitFromTranslateTaskID = RELATED_TRANSLATE_JOB_NOT_FOUND;
		return;
	}

	const FRHITranslateTask& PrevTask = Provider.GetRHITranslateTask(PrevTaskID);
	if (PrevTask.Type != ERHITranslateTaskType::Translate)
	{
		UE_LOGF(LogRenderTrace, Warning, "Translate task %llx (permanent ID %x) split from %llx: found previous task with permanent ID %x, but it's not a translate task.",
			Task.AppID, TaskID, FirstCmdList.TranslatePrevJobAppID, PrevTaskID);
		Task.SplitFromTranslateTaskID = RELATED_TRANSLATE_JOB_NOT_FOUND;
		return;
	}

	if (PrevTask.SplitToTranslateTaskID != INVALID_EVENT_ID && PrevTask.SplitToTranslateTaskID != TaskID)
	{
		UE_LOGF(LogRenderTrace, Warning, "Translate task %llx (permanent ID %x) split from %llx: found previous task with permanent ID %x which already has a different split to ID: %x.",
			Task.AppID, TaskID, FirstCmdList.TranslatePrevJobAppID, PrevTaskID, PrevTask.SplitToTranslateTaskID);
		Task.SplitFromTranslateTaskID = RELATED_TRANSLATE_JOB_NOT_FOUND;
		return;
	}

	Task.SplitFromTranslateTaskID = PrevTaskID;

	// Link the previous task to this.
	PrevTask.SplitToTranslateTaskID = TaskID;
}

void FRHIThreadTrack::CacheSplitToTask(const IRenderTraceProvider& Provider, uint32 TaskID) const
{
	// See the comment in CacheSplitFromTask about why it's OK to modify the task object without taking the writer lock.

	const FRHITranslateTask& Task = Provider.GetRHITranslateTask(TaskID);
	if (Task.SplitToTranslateTaskID < INVALID_EVENT_ID)
	{
		// Already cached.
		return;
	}

	// Find the closest task which says it split from TaskID.
	const uint32 NextTaskID = Provider.FindRHITaskByPredicate(TaskID, 0.05,
		[&Provider, &Task](const FRHITranslateTask& OtherTask) -> int
		{
			if (OtherTask.Type == ERHITranslateTaskType::Translate && !OtherTask.ProcessedItems.IsEmpty())
			{
				const FCommandListInstance& FirstCmdList = Provider.GetCommandList(OtherTask.ProcessedItems[0].ItemID);
				if (FirstCmdList.TranslatePrevJobAppID == Task.AppID) return 1;
			}

			// If we encounter another task with the same app ID as ourselves it's a reused pointer, so we've moved past our lifetime.
			return (OtherTask.AppID == Task.AppID) ? -1 : 0;
		}
	);

	if (NextTaskID == INVALID_EVENT_ID)
	{
		Task.SplitToTranslateTaskID = RELATED_TRANSLATE_JOB_NOT_FOUND;
		return;
	}

	const FRHITranslateTask& NextTask = Provider.GetRHITranslateTask(NextTaskID);
	check(NextTask.Type == ERHITranslateTaskType::Translate);

	// Due to pointer reuse, it might be that the next task we found is actually split from a different task which just happens
	// to have the same pointer value as the task we started with. Do another search starting at the next task to see if we arrive
	// back at the current task. If not, it's a false match caused by pointer reuse.
	CacheSplitFromTask(Provider, NextTaskID);
	if (NextTask.SplitFromTranslateTaskID != TaskID)
	{
		Task.SplitToTranslateTaskID = RELATED_TRANSLATE_JOB_NOT_FOUND;
		return;
	}

	Task.SplitToTranslateTaskID = NextTaskID;
}

void FRHIThreadTrack::PopulateTranslateJobDetails(IEventDetailsBuilder& Builder, uint32 TaskID, const FRHITranslateTask& Task) const
{
	const IRenderTraceProvider& Provider = Builder.GetProvider();

	Builder.AddTitle(FString::Printf(TEXT("Translate job %s"), *GetTranslateTaskName(Task)));

	{
		FString FlagList;
		ADD_FLAG_TO_LIST(Task.JobFlags, ERHITranslateJobFlag, Parallel);
		ADD_FLAG_TO_LIST(Task.JobFlags, ERHITranslateJobFlag, UsingSubCmdLists);
		if (FlagList.IsEmpty()) FlagList = TEXT("None");
		Builder.AddNameValueTextLine(TEXTVIEW("Flags:"), FlagList);
	}

	if (!Task.ProcessedItems.IsEmpty())
	{
		const FCommandListInstance& FirstCmdList = Provider.GetCommandList(Task.ProcessedItems[0].ItemID);
		FString FlagList;
		ADD_FLAG_TO_LIST(FirstCmdList.TranslateSplitFlags, ERHITranslateJobSplitFlag, Parallel);
		ADD_FLAG_TO_LIST(FirstCmdList.TranslateSplitFlags, ERHITranslateJobSplitFlag, Threshold);
		ADD_FLAG_TO_LIST(FirstCmdList.TranslateSplitFlags, ERHITranslateJobSplitFlag, ParentChild);
		ADD_FLAG_TO_LIST(FirstCmdList.TranslateSplitFlags, ERHITranslateJobSplitFlag, JumpThreads);
		if (FlagList.IsEmpty()) FlagList = TEXT("None");
		Builder.AddNameValueTextLine(TEXTVIEW("Split Reason:"), FlagList);

		if (FirstCmdList.TranslatePrevJobAppID > 0)
		{
			Builder.AddNameValueTextLine(TEXTVIEW("Split From:"), FString::Printf(TEXT("0x%llx"), FirstCmdList.TranslatePrevJobAppID));
		}
	}

	CacheSplitToTask(Provider, TaskID);
	if (Task.SplitToTranslateTaskID < RELATED_TRANSLATE_JOB_NOT_FOUND)
	{
		const FRHITranslateTask& SplitToTask = Provider.GetRHITranslateTask(Task.SplitToTranslateTaskID);
		Builder.AddNameValueTextLine(TEXTVIEW("Split To:"), GetTranslateTaskName(SplitToTask));
	}

	if (Task.NextPhaseTaskID != INVALID_EVENT_ID)
	{
		Builder.AddNameValueTextLine(TEXTVIEW("Submitted By:"), LexToString(Task.NextPhaseTaskID));
	}
	else
	{
		Builder.AddNameValueTextLine(TEXTVIEW("NOT SUBMITTED"), TEXTVIEW(""));
	}

	FAppIDListBuilder<FCommandListInstance>::AddAllEvents(Builder, TEXT("Command Lists:"), Task.ProcessedItems);

	if (Builder.GetDetailLevel() == EEventDetailLevel::Full)
	{
		Builder.AddNameValueTextLine(TEXTVIEW("RDG Passes:"), TEXTVIEW(""));
		for (const FEventInterval& CmdListEvent : Task.ProcessedItems)
		{
			const FCommandListInstance& CmdList = Provider.GetCommandList(CmdListEvent.ItemID);
			for (const FEventInterval& RecordingEvent : CmdList.RecordingEvents)
			{
				const FRDGPassInstance& Pass = Provider.GetRDGPass(RecordingEvent.ItemID);
				Builder.AddNameValueTextLine(TEXTVIEW(""), Pass.Name);
			}
		}
	}
}
void FRHIThreadTrack::ListSyncPoints(IEventDetailsBuilder& Builder, const TCHAR* Title, TConstArrayView<uint32> SyncPointIDs) const
{
	if (SyncPointIDs.IsEmpty())
	{
		return;
	}

	const IRenderTraceProvider& Provider = Builder.GetProvider();
	Builder.AddNameValueTextLine(Title, TEXTVIEW(""));
	for (uint32 SyncPointID : SyncPointIDs)
	{
		const FSyncPoint& SyncPoint = Provider.GetSyncPoint(SyncPointID);
		Builder.AddNameValueTextLine(TEXTVIEW(""), FString::Printf(TEXT("0x%llx (%s): %llu"), SyncPoint.AppID, GetSyncPointTypeName(SyncPoint.Type), SyncPoint.ResolvedValue));
	}
}

void FRHIThreadTrack::PopulatePayloadDetails(IEventDetailsBuilder& Builder, uint32 TranslateJobID, uint32 PipeIdx, uint32 PayloadIdx) const
{
	const IRenderTraceProvider& Provider = Builder.GetProvider();

	const FRHITranslateTask& TranslateJob = Provider.GetRHITranslateTask(TranslateJobID);
	check(TranslateJob.Type == ERHITranslateTaskType::Translate);
	const FRHITranslateContext& Context = TranslateJob.Contexts[PipeIdx];
	check(Context.RHIContextID != 0);
	const FPlatformPayload& Payload = Provider.GetPlatformPayload(Context.PayloadIDs[PayloadIdx]);

	Builder.AddTitle(FString::Printf(TEXT("Payload 0x%llx"), Payload.AppID));

	Builder.AddNameValueTextLine(TEXTVIEW("Pipe:"), GetPipeName(PipeIdx));
	Builder.AddNameValueTextLine(TEXTVIEW("Context ID:"), FString::Printf(TEXT("0x%llx"), Context.RHIContextID));

	{
		FAppIDListBuilder<uint64> ListBuilder(Builder, Payload.CmdLists.Num());
		Builder.AddNameValueTextLine(TEXTVIEW("Platform Command Lists:"), LexToString(Payload.CmdLists.Num()));
		for (uint64 CmdListID : Payload.CmdLists) { ListBuilder.AddItem(CmdListID); }
		ListBuilder.Finish();
	}

	ListSyncPoints(Builder, TEXT("Wait Sync Points:"), Payload.WaitSyncPoints);
	ListSyncPoints(Builder, TEXT("Signal Sync Points:"), Payload.SignalSyncPoints);
}

void FRHIThreadTrack::PopulateTranslateContextDetails(IEventDetailsBuilder& Builder, uint32 TranslateJobID, uint32 PipeIdx) const
{
	const IRenderTraceProvider& Provider = Builder.GetProvider();

	const FRHITranslateTask& TranslateJob = Provider.GetRHITranslateTask(TranslateJobID);
	check(TranslateJob.Type == ERHITranslateTaskType::Translate);
	const FRHITranslateContext& Context = TranslateJob.Contexts[PipeIdx];
	check(Context.RHIContextID != 0);

	Builder.AddTitle(FString::Printf(TEXT("Translate Context 0x%llx"), Context.RHIContextID));
	Builder.AddNameValueTextLine(TEXTVIEW("Pipe:"), GetPipeName(PipeIdx));

	uint32 NumCmdLists = 0;
	
	{
		FAppIDListBuilder<FPlatformPayload> ListBuilder(Builder, Context.PayloadIDs.Num());
		Builder.AddNameValueTextLine(TEXTVIEW("Payloads:"), LexToString(Context.PayloadIDs.Num()));
		for (uint32 ID : Context.PayloadIDs)
		{
			const FPlatformPayload& Payload = Provider.GetPlatformPayload(ID);
			ListBuilder.AddItem(Payload.AppID);
			NumCmdLists += Payload.CmdLists.Num();
		}
		ListBuilder.Finish();
	}

	{
		FAppIDListBuilder<uint64> ListBuilder(Builder, NumCmdLists);
		Builder.AddNameValueTextLine(TEXTVIEW("Platform Command Lists:"), LexToString(NumCmdLists));
		for (uint32 ID : Context.PayloadIDs)
		{
			const FPlatformPayload& Payload = Provider.GetPlatformPayload(ID);
			for (uint64 CmdListID: Payload.CmdLists)
			{
				ListBuilder.AddItem(CmdListID);
			}
		}
		ListBuilder.Finish();
	}
}

void FRHIThreadTrack::PopulateEventDetails(IEventDetailsBuilder& Builder, const FTimingEvent& Event) const
{
	const IRenderTraceProvider& Provider = Builder.GetProvider();

	const uint32 EventDepth = Event.GetDepth();

	switch (EventDepth)
	{
	case 0:
	{
		const uint32 TaskID = static_cast<uint32>(Event.GetType());
		const FRHITranslateTask& Task = Provider.GetRHITranslateTask(TaskID);
		check(Task.Type == ERHITranslateTaskType::Translate);
		PopulateTranslateJobDetails(Builder, TaskID, Task);
		return;
	}

	case 1:
	{
		const uint32 ItemID = static_cast<uint32>(Event.GetType());
		const uint32 ParentTaskID = static_cast<uint32>(Event.GetType() >> 32);
		const FRHITranslateTask& ParentTask = Provider.GetRHITranslateTask(ParentTaskID);
		check(ParentTask.Type == ERHITranslateTaskType::Translate);
		const FCommandListInstance& CmdList = Provider.GetCommandList(ItemID);
		Builder.AddTitle(FString::Printf(TEXT("Command List %s"), *GetCommandListName(CmdList)));
		return;
	}
	}

	const uint32 ItemID = static_cast<uint32>(Event.GetType());
	const uint32 ParentTaskID = static_cast<uint32>(Event.GetType() >> 32);

	const uint32 FirstContextDepth = 2;
	const uint32 ContextDepth = EventDepth - FirstContextDepth;
	const uint32 PipeIdx = ContextDepth / 2;
	const bool bIsPayload = (ContextDepth % 2) == 1;
	check(PipeIdx < 2);

	if (bIsPayload)
	{
		PopulatePayloadDetails(Builder, ParentTaskID, PipeIdx, ItemID);
	}
	else
	{
		PopulateTranslateContextDetails(Builder, ParentTaskID, PipeIdx);
	}
}

void FRHIThreadTrack::OnTranslateEventSelected(const FTimingEvent& InSelectedEvent) const
{
	UE::Insights::Timing::ITimingViewSession* TimingView = SharedData.GetTimingViewSession();
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	check(TimingView && Provider);

	const uint32 TaskID = static_cast<uint32>(InSelectedEvent.GetType());
	const FRHITranslateTask& Task = Provider->GetRHITranslateTask(TaskID);
	check(Task.Type == ERHITranslateTaskType::Translate);

	// Add an arrow to the CPU thread where the translation job executed.
	TSharedPtr<const FBaseTimingTrack> CpuThreadTrack = SharedData.GetCpuThreadTrack(Task.ThreadID);
	if (CpuThreadTrack)
	{
		TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
			SharedThis(this), Task.StartTime, 0, CpuThreadTrack, Task.StartTime, 0, GRelationColorExecThread
		);
		TimingView->AddRelation(Relation);
	}

	// Draw arrows from all the command lists which are translated by this job.
	for (int32 CmdListEventIdx = 0; CmdListEventIdx < Task.ProcessedItems.Num(); ++CmdListEventIdx)
	{
		const FEventInterval& CmdListEvent = Task.ProcessedItems[CmdListEventIdx];
		const FCommandListInstance& CmdList = Provider->GetCommandList(CmdListEvent.ItemID);

		if (CmdList.TimelineIndex < 0 || CmdList.FinishRecordingTime == 0.0)
		{
			continue;
		}

		TSharedPtr<const FCommandListTrack> CmdListTrack = SharedData.GetCommandListTrack(CmdList.TimelineIndex);
		if (!CmdListTrack.IsValid())
		{
			continue;
		}

		TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
			CmdListTrack, CmdList.FinishRecordingTime, 0, SharedThis(this), CmdListEvent.Start, 1, GRelationColorTranslate
		);

		TimingView->AddRelation(Relation);
	}

	// Draw arrow to our portion of the submit task.
	if (Task.NextPhaseTaskID != INVALID_EVENT_ID)
	{
		const FRHITranslateTask& SubmitTask = Provider->GetRHITranslateTask(Task.NextPhaseTaskID);
		check(SubmitTask.Type == ERHITranslateTaskType::Submit);

		TSharedPtr<const FBaseTimingTrack> RHISubmissionTrack = GetFirstChildTrack(SharedData.GetRHIThreadTrack(SubmitTask.ThreadID));
		if (RHISubmissionTrack.IsValid())
		{
			for (const FEventInterval& TranslateEvent : SubmitTask.ProcessedItems)
			{
				if (TranslateEvent.ItemID == TaskID)
				{
					TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
						SharedThis(this), Task.EndTime, 0, RHISubmissionTrack, TranslateEvent.Start, 1, GRelationColorSubmitTranslateJob
					);

					TimingView->AddRelation(Relation);
					break;
				}
			}
		}
	}

	// If we triggered submission, draw an arrow to the submission event.
	if (Task.SubmitTaskID != INVALID_EVENT_ID)
	{
		const FRHITranslateTask& SubmitTask = Provider->GetRHITranslateTask(Task.SubmitTaskID);
		check(SubmitTask.Type == ERHITranslateTaskType::Submit);
		TSharedPtr<const FBaseTimingTrack> RHISubmissionTrack = GetFirstChildTrack(SharedData.GetRHIThreadTrack(SubmitTask.ThreadID));
		if (RHISubmissionTrack.IsValid())
		{
			TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
				SharedThis(this), Task.EndTime, 0, RHISubmissionTrack, SubmitTask.StartTime, 0, GRelationColorTriggerSubmission
			);

			TimingView->AddRelation(Relation);
		}
	}
	
	CacheSplitFromTask(*Provider, TaskID);
	if (Task.SplitFromTranslateTaskID < RELATED_TRANSLATE_JOB_NOT_FOUND)
	{
		// Draw arrow from translate job which was split into this job.
		const FRHITranslateTask& PrevTask = Provider->GetRHITranslateTask(Task.SplitFromTranslateTaskID);
		check(PrevTask.Type == ERHITranslateTaskType::Translate);
		TSharedPtr<const FRHIThreadTrack> PrevTaskTrack = SharedData.GetRHIThreadTrack(PrevTask.ThreadID);
		if (PrevTaskTrack.IsValid())
		{
			TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
				PrevTaskTrack, PrevTask.EndTime, 0, SharedThis(this), Task.StartTime, 0, GRelationColorSplitFromTranslateJob
			);

			TimingView->AddRelation(Relation);
		}
	}

	CacheSplitToTask(*Provider, TaskID);
	if (Task.SplitToTranslateTaskID < RELATED_TRANSLATE_JOB_NOT_FOUND)
	{
		// Draw arrow to translate job which followed this task after a split.
		const FRHITranslateTask& NextTask = Provider->GetRHITranslateTask(Task.SplitToTranslateTaskID);
		check(NextTask.Type == ERHITranslateTaskType::Translate);
		TSharedPtr<const FRHIThreadTrack> NextTaskTrack = SharedData.GetRHIThreadTrack(NextTask.ThreadID);
		if (NextTaskTrack.IsValid())
		{
			TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
				SharedThis(this), Task.EndTime, 0, NextTaskTrack, NextTask.StartTime, 0, GRelationColorSplitToTranslateJob
			);

			TimingView->AddRelation(Relation);
		}
	}
}
void FRHIThreadTrack::OnTranslateCommandListEventSelected(const FTimingEvent& InSelectedEvent) const
{
	UE::Insights::Timing::ITimingViewSession* TimingView = SharedData.GetTimingViewSession();
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	check(TimingView && Provider);

	// Draw arrow from the command list to the translate event inside the job task.
	const uint32 CmdListID = static_cast<uint32>(InSelectedEvent.GetType());
	const FCommandListInstance& CmdList = Provider->GetCommandList(CmdListID);
	if (CmdList.TimelineIndex < 0 || CmdList.FinishRecordingTime == 0.0)
	{
		return;
	}

	TSharedPtr<const FCommandListTrack> CmdListTrack = SharedData.GetCommandListTrack(CmdList.TimelineIndex);
	if (!CmdListTrack.IsValid())
	{
		return;
	}

	TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
		CmdListTrack, CmdList.FinishRecordingTime, 0, SharedThis(this), InSelectedEvent.GetStartTime(), 1, GRelationColorTranslate
	);

	TimingView->AddRelation(Relation);
}
void FRHIThreadTrack::OnPayloadEventSelected(const FTimingEvent& InSelectedEvent, uint32 TranslateJobID, uint8 PipeIdx, uint32 PayloadIdx) const
{
	UE::Insights::Timing::ITimingViewSession* TimingView = SharedData.GetTimingViewSession();
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	check(TimingView && Provider);

	const FRHITranslateTask& TranslateTask = Provider->GetRHITranslateTask(TranslateJobID);
	check(TranslateTask.Type == ERHITranslateTaskType::Translate);

	const uint32 PayloadID = TranslateTask.Contexts[PipeIdx].PayloadIDs[PayloadIdx];
	const FPlatformPayload& Payload = Provider->GetPlatformPayload(PayloadID);

	TSharedPtr<const FSubmissionQueueTrack> SubmissionTrack = SharedData.GetSubmissionTrack();
	if (SubmissionTrack.IsValid())
	{
		// Draw arrow to the submission event which executed the payload.
		if (Payload.ExecutionEvent.ParentItemID != INVALID_EVENT_ID)
		{
			const FSubmissionBatch& SubmissionBatch = Provider->GetSubmissionBatch(Payload.ExecutionEvent.ParentItemID);
			const FSubmissionEvent& ExecuteEvent = SubmissionBatch.Events[Payload.ExecutionEvent.ItemID];
			check(ExecuteEvent.Data.GetIndex() == ESubmissionEvent_Execute);

			TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
				SharedThis(this), InSelectedEvent.GetEndTime(), InSelectedEvent.GetDepth(), SubmissionTrack, ExecuteEvent.Timestamp, 1, GRelationColorExecute
			);

			TimingView->AddRelation(Relation);
		}

		// Draw arrows to all resolve sync point events.
		for (const FRenderTraceEvent& ResolveEventLocator : Payload.ResolveSyncPointEvents)
		{
			const FSubmissionBatch& SubmissionBatch = Provider->GetSubmissionBatch(ResolveEventLocator.ParentItemID);
			const FSubmissionEvent& ResolveEvent = SubmissionBatch.Events[ResolveEventLocator.ItemID];
			check(ResolveEvent.Data.GetIndex() == ESubmissionEvent_ResolveSyncPoint);

			TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
				SharedThis(this), InSelectedEvent.GetEndTime(), InSelectedEvent.GetDepth(), SubmissionTrack, ResolveEvent.Timestamp, 1, GRelationColorResolveSyncPoint
			);

			TimingView->AddRelation(Relation);
		}
	}

	// Draw arrow to the interrupt event which signaled the payload as completed.
	TSharedPtr<const FInterruptTrack> InterruptTrack = SharedData.GetInterruptTrack();
	if (InterruptTrack.IsValid() && Payload.InterruptEvent.ParentItemID != INVALID_EVENT_ID)
	{
		const FInterruptWakeUp& WakeUp = Provider->GetInterruptWakeUp(Payload.InterruptEvent.ParentItemID);
		const FInterruptFenceSignaledEvent& SignalEvent = WakeUp.SignalEvents[Payload.InterruptEvent.ItemID];

		TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
			SharedThis(this), InSelectedEvent.GetEndTime(), InSelectedEvent.GetDepth(), InterruptTrack, SignalEvent.Timestamp, 1, GRelationColorInterrupt
		);

		TimingView->AddRelation(Relation);
	}
}

void FRHIThreadTrack::OnRenderTraceEventSelected(const FTimingEvent& InSelectedEvent) const
{
	UE::Insights::Timing::ITimingViewSession* TimingView = SharedData.GetTimingViewSession();
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	if (!Provider || !TimingView)
	{
		return;
	}

	const uint32 Depth = InSelectedEvent.GetDepth();
	const uint32 ParentItemID = static_cast<uint32>(InSelectedEvent.GetType() >> 32);
	const uint32 ItemID = static_cast<uint32>(InSelectedEvent.GetType());

	switch (Depth)
	{
	case 0:
	{
		const FRHITranslateTask& Task = Provider->GetRHITranslateTask(ItemID);
		check(Task.Type == ERHITranslateTaskType::Translate);
		OnTranslateEventSelected(InSelectedEvent);
		return;
	}

	case 1:
	{
		check(ParentItemID != INVALID_EVENT_ID);
		const FRHITranslateTask& ParentTask = Provider->GetRHITranslateTask(ParentItemID);
		check(ParentTask.Type == ERHITranslateTaskType::Translate);
		OnTranslateCommandListEventSelected(InSelectedEvent);
		return;
	}
	}

	const uint32 FirstContextDepth = 2;
	const uint32 ContextDepth = Depth - FirstContextDepth;
	const uint32 PipeIdx = ContextDepth / 2;
	if (PipeIdx > 1)
	{
		return;
	}

	const bool bIsPayload = (ContextDepth % 2) == 1;
	if (bIsPayload)
	{
		OnPayloadEventSelected(InSelectedEvent, ParentItemID, PipeIdx, ItemID);
	}
}

FString FRHIThreadTrack::GetEventName(const IRenderTraceProvider* Provider, uint32 InDepth, const FRenderTraceEvent& InEvent) const
{
	switch (InDepth)
	{
	case 0:
	{
		const FRHITranslateTask& Task = Provider->GetRHITranslateTask(InEvent.ItemID);
		check(Task.Type == ERHITranslateTaskType::Translate);
		return FString::Printf(TEXT("Translate %s"), *GetTranslateTaskName(Task));
	}

	case 1:
	{
		check(InEvent.ParentItemID != INVALID_EVENT_ID);
		const FRHITranslateTask& ParentTask = Provider->GetRHITranslateTask(InEvent.ParentItemID);
		check(ParentTask.Type == ERHITranslateTaskType::Translate);
		const FCommandListInstance& CmdList = Provider->GetCommandList(InEvent.ItemID);
		return FString::Printf(TEXT("CMD %s"), *GetCommandListName(CmdList));
	}
	}

	checkNoEntry();
	return TEXT("invalid");
}

void FRHIThreadTrack::CustomizeEventDrawState(ITimingEventsTrackDrawStateBuilder& Builder, double StartTime, double EndTime, uint32 Depth, const FRenderTraceEvent& Event) const
{
	// For each translate job event we will add context and payload events starting at depth 2.
	if (Depth > 0)
	{
		return;
	}

	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	if (!Provider)
	{
		return;
	}

	const FRHITranslateTask& Task = Provider->GetRHITranslateTask(Event.ItemID);
	if (Task.Type != ERHITranslateTaskType::Translate || Task.ProcessedItems.IsEmpty())
	{
		return;
	}

	for (int PipeIdx = 0; PipeIdx < 2; ++PipeIdx)
	{
		const int ContextDepth = 2 + 2 * PipeIdx;
		const FRHITranslateContext& Context = Task.Contexts[PipeIdx];
		const FString ContextName = FString::Printf(TEXT("CTX 0x%llx"), Context.RHIContextID);

		for (const FTimeInterval& Interval : Context.ActiveIntervals)
		{
			const double CtxEndTime = Interval.End != 0 ? FMath::Min(Interval.End, EndTime) : EndTime;
			Builder.AddEvent(Interval.Start, CtxEndTime, ContextDepth, *ContextName);
		}

		for (uint32 PayloadID : Context.PayloadIDs)
		{
			const FPlatformPayload& Payload = Provider->GetPlatformPayload(PayloadID);
			FString PayloadName = FString::Printf(TEXT("Payload 0x%llx"), Payload.AppID);
			Builder.AddEvent(Payload.StartTime, Payload.EndTime, ContextDepth + 1, *PayloadName);
		}
	}
}

TSharedPtr<FTimingEvent> FRHIThreadTrack::GetEventAtTime(double EventTime, int32 EventDepth, double TimelineEventStart, double TimelineEventEnd, int32 TimelineEventDepth, const FRenderTraceEvent& TimelineEvent) const
{
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	if (!Provider)
	{
		return nullptr;
	}

	// We customize the context and platform context events, which start under the job and command list lanes.
	const int32 FirstContextDepth = 2;
	if (EventDepth < FirstContextDepth)
	{
		return nullptr;
	}

	// The information for these events is based on the translate job, which is at depth 0 on the timeline.
	if (TimelineEventDepth > 0)
	{
		return nullptr;
	}

	// We have two lanes per pipe: context and platform command lists.
	const int32 ContextDepth = EventDepth - FirstContextDepth;
	const int32 PipeIdx = ContextDepth / 2;
	if (PipeIdx >= 2)
	{
		return nullptr;
	}

	const bool bIsPayload = (ContextDepth % 2) == 1;

	const FRHITranslateTask& TranslateJob = Provider->GetRHITranslateTask(TimelineEvent.ItemID);
	if (TranslateJob.Type != ERHITranslateTaskType::Translate)
	{
		return nullptr;
	}

	const FRHITranslateContext& Context = TranslateJob.Contexts[PipeIdx];
	if (Context.ActiveIntervals.IsEmpty())
	{
		return nullptr;
	}

	// For payloads, the item ID stores the payload index. For contexts we don't need to store anything, since the
	// depth tells us if it's a context of a payload.
	uint32 ItemID = INVALID_EVENT_ID;

	double EventStart = TimelineEventStart, EventEnd = TimelineEventEnd;

	if (bIsPayload)
	{
		// We must find the payload which was active at EventTime. We usually have a small number of payloads,
		// so a linear search is good enough.
		for (int PayloadIdx = 0; PayloadIdx < Context.PayloadIDs.Num(); ++PayloadIdx)
		{
			const FPlatformPayload& Payload = Provider->GetPlatformPayload(Context.PayloadIDs[PayloadIdx]);
			if (EventTime >= Payload.StartTime && EventTime <= Payload.EndTime)
			{
				ItemID = PayloadIdx;
				EventStart = Payload.StartTime;
				EventEnd = Payload.EndTime;
				break;
			}
		}

		if (ItemID == INVALID_EVENT_ID)
		{
			return nullptr;
		}
	}
	else
	{
		// We create a single event for the entire lifetime of the context instead of individual events for each
		// activity interval, to highlight that it's in fact a single context for the entire lane.
		EventStart = Context.ActiveIntervals[0].Start;
		EventEnd = Context.ActiveIntervals.Last().End;
	}

	EventEnd = (EventEnd != 0.0) ? FMath::Min(EventEnd, TranslateJob.EndTime) : TranslateJob.EndTime;

	// Use the translate job ID as the parent ID.
	const uint64 Packed = ((uint64)TimelineEvent.ItemID << 32) | ItemID;
	return MakeShared<FTimingEvent>(SharedThis(this), EventStart, EventEnd, EventDepth, Packed);
}

FRHISubmissionTrack::FRHISubmissionTrack(const FRenderTraceTimingViewSession& InSharedData, const FString& Name, const IRenderTraceProvider::TEventTimeline* InTimeline)
	: FRenderTraceTrack(InSharedData, Name, InTimeline)
{
}

void FRHISubmissionTrack::OnSubmitEventSelected(const FTimingEvent& InSelectedEvent) const
{
	UE::Insights::Timing::ITimingViewSession* TimingView = SharedData.GetTimingViewSession();
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	check(TimingView && Provider);

	const uint32 SubmitTaskID = static_cast<uint32>(InSelectedEvent.GetType());
	const FRHITranslateTask& SubmitTask = Provider->GetRHITranslateTask(SubmitTaskID);
	check(SubmitTask.Type == ERHITranslateTaskType::Submit);

	// Draw arrow from instigating task.
	if (SubmitTask.SubmitTaskID != INVALID_EVENT_ID)
	{
		TSharedPtr<const FBaseTimingTrack> InstigatorTrack;
		double InstigatorTime;
		if (SubmitTask.bEagerSubmission)
		{
			// The instigator is a translate job.
			const FRHITranslateTask& InstigatorTask = Provider->GetRHITranslateTask(SubmitTask.SubmitTaskID);
			check(InstigatorTask.Type == ERHITranslateTaskType::Translate);
			InstigatorTrack = SharedData.GetRHIThreadTrack(InstigatorTask.ThreadID);
			InstigatorTime = InstigatorTask.EndTime;
		}
		else
		{
			// The instigator is a render thread submit event.
			const FRDGPassInstance& SubmitPass = Provider->GetRDGPass(SubmitTask.SubmitTaskID);
			InstigatorTrack = GetFirstChildTrack(SharedData.GetRDGThreadTrack(SubmitPass.ExecThreadID));
			InstigatorTime = SubmitPass.EndTime;
		}

		if (InstigatorTrack.IsValid())
		{
			TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
				InstigatorTrack, InstigatorTime, 0, SharedThis(this), SubmitTask.StartTime, 0, GRelationColorTriggerSubmission
			);
			TimingView->AddRelation(Relation);
		}
	}

	// Draw arrows from all translate jobs which are submitted.
	for (const FEventInterval& TranslateEvent : SubmitTask.ProcessedItems)
	{
		const FRHITranslateTask& TranslateTask = Provider->GetRHITranslateTask(TranslateEvent.ItemID);
		check(TranslateTask.Type == ERHITranslateTaskType::Translate);
		TSharedPtr<const FRHIThreadTrack> TranslateTrack = SharedData.GetRHIThreadTrack(TranslateTask.ThreadID);
		if (TranslateTrack.IsValid())
		{
			TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
				TranslateTrack, TranslateTask.EndTime, 0, SharedThis(this), TranslateEvent.Start, 1, GRelationColorSubmitTranslateJob
			);

			TimingView->AddRelation(Relation);
		}
	}
}

void FRHISubmissionTrack::OnSubmitTranslateJobEventSelected(const FTimingEvent& InSelectedEvent) const
{
	UE::Insights::Timing::ITimingViewSession* TimingView = SharedData.GetTimingViewSession();
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	check(TimingView && Provider);

	// Draw arrow from the translate job to the event inside the submit task.
	const uint32 TranslateTaskID = static_cast<uint32>(InSelectedEvent.GetType());
	const FRHITranslateTask& TranslateTask = Provider->GetRHITranslateTask(TranslateTaskID);
	check(TranslateTask.Type == ERHITranslateTaskType::Translate);

	TSharedPtr<const FRHIThreadTrack> TranslateTrack = SharedData.GetRHIThreadTrack(TranslateTask.ThreadID);
	if (!TranslateTrack.IsValid())
	{
		return;
	}

	TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
		TranslateTrack, TranslateTask.EndTime, 0, SharedThis(this), InSelectedEvent.GetStartTime(), 1, GRelationColorSubmitTranslateJob
	);

	TimingView->AddRelation(Relation);
}

void FRHISubmissionTrack::OnRenderTraceEventSelected(const FTimingEvent& InSelectedEvent) const
{
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	UE::Insights::Timing::ITimingViewSession* TimingView = SharedData.GetTimingViewSession();
	if (!Provider || !TimingView)
	{
		return;
	}

	const uint32 Depth = InSelectedEvent.GetDepth();

	switch (Depth)
	{
	case 0:
	{
		const uint32 TaskID = static_cast<uint32>(InSelectedEvent.GetType());
		const FRHITranslateTask& Task = Provider->GetRHITranslateTask(TaskID);
		check(Task.Type == ERHITranslateTaskType::Submit);
		OnSubmitEventSelected(InSelectedEvent);
		return;
	}

	case 1:
	{
		const uint32 ParentItemID = static_cast<uint32>(InSelectedEvent.GetType() >> 32);
		check(ParentItemID != INVALID_EVENT_ID);
		const FRHITranslateTask& ParentTask = Provider->GetRHITranslateTask(ParentItemID);
		check(ParentTask.Type == ERHITranslateTaskType::Submit);
		OnSubmitTranslateJobEventSelected(InSelectedEvent);
		return;
	}
	}

	checkNoEntry();
}

FString FRHISubmissionTrack::GetEventName(const IRenderTraceProvider* Provider, uint32 InDepth, const FRenderTraceEvent& InEvent) const
{
	switch (InDepth)
	{
	case 0:
	{
		const FRHITranslateTask& Task = Provider->GetRHITranslateTask(InEvent.ItemID);
		check(Task.Type == ERHITranslateTaskType::Submit);
		return FString::Printf(TEXT("Submit %s"), *GetTranslateTaskName(Task));
	}

	case 1:
	{
		check(InEvent.ParentItemID != INVALID_EVENT_ID);
		const FRHITranslateTask& TranslateTask = Provider->GetRHITranslateTask(InEvent.ItemID);
		return GetTranslateTaskName(TranslateTask);
	}
	}

	checkNoEntry();
	return TEXT("invalid");
}

void FRHISubmissionTrack::PopulateSubmitTaskDetails(IEventDetailsBuilder& Builder, uint32 TaskID, const FRHITranslateTask& Task) const
{
	const IRenderTraceProvider& Provider = Builder.GetProvider();

	Builder.AddTitle(TEXTVIEW("Submit Task"));
	Builder.AddNameValueTextLine(TEXTVIEW("ID:"), LexToString(Task.AppID));

	if (Task.SubmitTaskID != INVALID_EVENT_ID)
	{
		if (Task.bEagerSubmission)
		{
			const FRHITranslateTask& InstigatorTask = Provider.GetRHITranslateTask(Task.SubmitTaskID);
			check(InstigatorTask.Type == ERHITranslateTaskType::Translate);
			Builder.AddNameValueTextLine(TEXTVIEW("Triggered By:"), FString::Printf(TEXT("Translate job 0x%llx"), InstigatorTask.AppID));
		}
		else
		{
			Builder.AddNameValueTextLine(TEXTVIEW("Triggered By:"), FString::Printf(TEXT("RT submit event %u"), Task.SubmitTaskID));
		}
	}
	else
	{
		Builder.AddNameValueTextLine(TEXTVIEW("NO INSTIGATOR INFO"), TEXTVIEW(""));
	}

	FAppIDListBuilder<FRHITranslateTask>::AddAllEvents(Builder, TEXT("Translate Jobs:"), Task.ProcessedItems);
	
	uint32 NumCmdLists = 0, NumPasses = 0;
	for (const FEventInterval& TranslateTaskEvent : Task.ProcessedItems)
	{
		const FRHITranslateTask& TranslateTask = Provider.GetRHITranslateTask(TranslateTaskEvent.ItemID);
		NumCmdLists += TranslateTask.ProcessedItems.Num();
		for (const FEventInterval& CmdListEvent : TranslateTask.ProcessedItems)
		{
			const FCommandListInstance& CmdList = Provider.GetCommandList(CmdListEvent.ItemID);
			NumPasses += CmdList.RecordingEvents.Num();
		}
	}

	Builder.AddNameValueTextLine(TEXTVIEW("Command Lists:"), LexToString(NumCmdLists));
	FAppIDListBuilder<FCommandListInstance> CmdListNames(Builder, NumCmdLists);
	for (const FEventInterval& TranslateTaskEvent : Task.ProcessedItems)
	{
		const FRHITranslateTask& TranslateTask = Provider.GetRHITranslateTask(TranslateTaskEvent.ItemID);
		CmdListNames.AddEvents(TranslateTask.ProcessedItems, false);
	}
	CmdListNames.Finish();

	if (Builder.GetDetailLevel() == EEventDetailLevel::Full)
	{
		Builder.AddNameValueTextLine(TEXTVIEW("RDG Passes:"), LexToString(NumPasses));
		for (const FEventInterval& TranslateTaskEvent : Task.ProcessedItems)
		{
			const FRHITranslateTask& TranslateTask = Provider.GetRHITranslateTask(TranslateTaskEvent.ItemID);
			for (const FEventInterval& CmdListEvent : TranslateTask.ProcessedItems)
			{
				const FCommandListInstance& CmdList = Provider.GetCommandList(CmdListEvent.ItemID);
				for (const FEventInterval& RecordingEvent : CmdList.RecordingEvents)
				{
					const FRDGPassInstance& Pass = Provider.GetRDGPass(RecordingEvent.ItemID);
					Builder.AddNameValueTextLine(TEXTVIEW(""), Pass.Name);
				}
			}
		}
	}
}

void FRHISubmissionTrack::PopulateEventDetails(IEventDetailsBuilder& Builder, const FTimingEvent& Event) const
{
	const IRenderTraceProvider& Provider = Builder.GetProvider();

	const uint32 EventDepth = Event.GetDepth();

	switch (EventDepth)
	{
	case 0:
	{
		const uint32 TaskID = static_cast<uint32>(Event.GetType());
		const FRHITranslateTask& Task = Provider.GetRHITranslateTask(TaskID);
		check(Task.Type == ERHITranslateTaskType::Submit);
		PopulateSubmitTaskDetails(Builder, TaskID, Task);
		return;
	}

	case 1:
	{
		const uint32 ItemID = static_cast<uint32>(Event.GetType());
		const uint32 ParentTaskID = static_cast<uint32>(Event.GetType() >> 32);
		const FRHITranslateTask& ParentTask = Provider.GetRHITranslateTask(ParentTaskID);
		check(ParentTask.Type == ERHITranslateTaskType::Submit);
		const FRHITranslateTask& Task = Provider.GetRHITranslateTask(ItemID);
		Builder.AddTitle(FString::Printf(TEXT("Translate job %s"), *GetTranslateTaskName(Task)));
		return;
	}
	}

	checkNoEntry();
}

FSubmissionQueueTrack::FSubmissionQueueTrack(const FRenderTraceTimingViewSession& InSharedData, const FString& Name, const IRenderTraceProvider::TEventTimeline* InTimeline)
	: FRenderTraceTrack(InSharedData, Name, InTimeline)
{
}

void FSubmissionQueueTrack::OnSubmissionBatchSelected(const FTimingEvent& InSelectedEvent, uint32 BatchID) const
{
	UE::Insights::Timing::ITimingViewSession* TimingView = SharedData.GetTimingViewSession();
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	check(TimingView && Provider);

	const FSubmissionBatch& Batch = Provider->GetSubmissionBatch(BatchID);

	// Add an arrow to the CPU thread where batch was processed.
	TSharedPtr<const FBaseTimingTrack> CpuThreadTrack = SharedData.GetCpuThreadTrack(Batch.ThreadID);
	if (CpuThreadTrack)
	{
		TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
			SharedThis(this), Batch.StartTime, 0, CpuThreadTrack, Batch.StartTime, 0, GRelationColorExecThread
		);
		TimingView->AddRelation(Relation);
	}
}

void FSubmissionQueueTrack::OnResolveSyncPointEventSelected(const FTimingEvent& InSelectedEvent, const FSubmissionEvent::FResolveSyncPoint& ResolveEvent) const
{
	UE::Insights::Timing::ITimingViewSession* TimingView = SharedData.GetTimingViewSession();
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	check(TimingView && Provider);

	const FSyncPoint& SyncPoint = Provider->GetSyncPoint(ResolveEvent.SyncPointID);
	if(SyncPoint.ResolvedByPayload == INVALID_EVENT_ID)
	{
		return;
	}
	
	// Draw arrow from the payload which resolved the sync point.
	const FPlatformPayload& Payload = Provider->GetPlatformPayload(SyncPoint.ResolvedByPayload);
	const FRHITranslateTask& TranslateTask = Provider->GetRHITranslateTask(Payload.TranslateEvent.ParentItemID);
	check(TranslateTask.Type == ERHITranslateTaskType::Translate);

	TSharedPtr<const FRHIThreadTrack> TranslateTrack = SharedData.GetRHIThreadTrack(TranslateTask.ThreadID);
	if (!TranslateTrack.IsValid())
	{
		return;
	}

	const uint32 FirstContextDepth = 2;
	const uint32 SourceDepth = FirstContextDepth + 2 * Payload.PipeIdx + 1;

	TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
		TranslateTrack, Payload.EndTime, SourceDepth, SharedThis(this), InSelectedEvent.GetStartTime(), 1, GRelationColorResolveSyncPoint
	);

	TimingView->AddRelation(Relation);
}

void FSubmissionQueueTrack::OnExecuteEventSelected(const FTimingEvent& InSelectedEvent, const FSubmissionEvent::FExecute& ExecEvent) const
{
	UE::Insights::Timing::ITimingViewSession* TimingView = SharedData.GetTimingViewSession();
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	check(TimingView && Provider);

	for (uint32 PayloadID : ExecEvent.PayloadIDs)
	{
		const FPlatformPayload& Payload = Provider->GetPlatformPayload(PayloadID);
		if (Payload.TranslateEvent.ParentItemID == INVALID_EVENT_ID)
		{
			continue;
		}

		const FRHITranslateTask& TranslateTask = Provider->GetRHITranslateTask(Payload.TranslateEvent.ParentItemID);
		check(TranslateTask.Type == ERHITranslateTaskType::Translate);

		// Draw arrow from the translate event which produced the payload.
		TSharedPtr<const FRHIThreadTrack> TranslateTrack = SharedData.GetRHIThreadTrack(TranslateTask.ThreadID);
		if (!TranslateTrack.IsValid())
		{
			continue;
		}

		const uint32 FirstContextDepth = 2;
		const uint32 SourceDepth = FirstContextDepth + 2 * Payload.PipeIdx + 1;

		TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
			TranslateTrack, Payload.EndTime, SourceDepth, SharedThis(this), InSelectedEvent.GetStartTime(), 1, GRelationColorExecute
		);

		TimingView->AddRelation(Relation);
	}
}

void FSubmissionQueueTrack::OnRenderTraceEventSelected(const FTimingEvent& InSelectedEvent) const
{
	UE::Insights::Timing::ITimingViewSession* TimingView = SharedData.GetTimingViewSession();
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	if (!Provider || !TimingView)
	{
		return;
	}

	const uint32 Depth = InSelectedEvent.GetDepth();
	const uint32 ParentItemID = static_cast<uint32>(InSelectedEvent.GetType() >> 32);
	const uint32 ItemID = static_cast<uint32>(InSelectedEvent.GetType());

	if (Depth == 0)
	{
		OnSubmissionBatchSelected(InSelectedEvent, ItemID);
		return;
	}

	check(Depth == 1);

	const FSubmissionBatch& Batch = Provider->GetSubmissionBatch(ParentItemID);
	const FSubmissionEvent& Event = Batch.Events[ItemID];

	switch (Event.Data.GetIndex())
	{
	case ESubmissionEvent_ResolveSyncPoint: OnResolveSyncPointEventSelected(InSelectedEvent, Event.Data.Get<FSubmissionEvent::FResolveSyncPoint>()); return;
	case ESubmissionEvent_Execute: OnExecuteEventSelected(InSelectedEvent, Event.Data.Get<FSubmissionEvent::FExecute>()); return;
	}
}

static FString GetSubmissionEventTypeName(const FSubmissionEvent& SubmissionEvent)
{
	const TCHAR* TypeName;
	switch (SubmissionEvent.Data.GetIndex())
	{
	case ESubmissionEvent_YieldSyncPoint:     TypeName = TEXT("Yield SyncPoint"); break;
	case ESubmissionEvent_YieldManualFence:   TypeName = TEXT("Yield Manual Fence"); break;
	case ESubmissionEvent_WaitQueueFence:     TypeName = TEXT("Wait Queue Fence"); break;
	case ESubmissionEvent_WaitManualFence:    TypeName = TEXT("Wait Manual Fence"); break;
	case ESubmissionEvent_Execute:            TypeName = TEXT("Execute"); break;
	case ESubmissionEvent_SignalManualFence:  TypeName = TEXT("Signal Manual Fence"); break;
	case ESubmissionEvent_SignalQueueFence:   TypeName = TEXT("Signal Queue Fence"); break;
	case ESubmissionEvent_ResolveSyncPoint:   TypeName = TEXT("Resolve Sync Point"); break;
	default:                                  TypeName = TEXT("UNKNOWN"); break;
	}
	return FString::Printf(TEXT("[%s] %s"), GetPipeName(SubmissionEvent.Queue), TypeName);
}

FString FSubmissionQueueTrack::GetEventName(const IRenderTraceProvider* Provider, uint32 InDepth, const FRenderTraceEvent& InEvent) const
{
	if (InDepth == 0)
	{
		return TEXT("ProcessSubmissionQueue");
	}

	const FSubmissionBatch& Batch = Provider->GetSubmissionBatch(InEvent.ParentItemID);
	const FSubmissionEvent& SubmissionEvent = Batch.Events[InEvent.ItemID];
	return GetSubmissionEventTypeName(SubmissionEvent);
}

void FSubmissionQueueTrack::PopulateSubmissionBatchDetails(IEventDetailsBuilder& Builder, uint32 ItemID) const
{
	const IRenderTraceProvider& Provider = Builder.GetProvider();
	const FSubmissionBatch& Batch = Provider.GetSubmissionBatch(ItemID);

	Builder.AddTitle(TEXTVIEW("ProcessSubmissionQueue"));
	Builder.AddNameValueTextLine(TEXTVIEW("Exit Status:"), GetExitStatusStr(Batch.ExitStatus));

	Builder.AddNameValueTextLine(TEXTVIEW("Events:"), LexToString(Batch.Events.Num()));

	uint32 NumPayloads = 0, NumCmdLists = 0;
	for (const FSubmissionEvent& Event : Batch.Events)
	{
		if (Event.Data.IsType<FSubmissionEvent::FExecute>())
		{
			const FSubmissionEvent::FExecute& ECL = Event.Data.Get<FSubmissionEvent::FExecute>();
			NumPayloads += ECL.PayloadIDs.Num();
			NumCmdLists += ECL.PlatformCmdListIDs.Num();
		}
	}

	Builder.AddNameValueTextLine(TEXTVIEW("Executed Payloads:"), LexToString(NumPayloads));
	Builder.AddNameValueTextLine(TEXTVIEW("Executed CmdLists:"), LexToString(NumCmdLists));

	if (Builder.GetDetailLevel() == EEventDetailLevel::Brief)
	{
		return;
	}

	for (const FSubmissionEvent& Event : Batch.Events)
	{
		const FString EventLabel = GetSubmissionEventTypeName(Event) + TEXT(":");

		switch (Event.Data.GetIndex())
		{
		case ESubmissionEvent_YieldSyncPoint:
		{
			const FSubmissionEvent::FYieldSyncPoint& YieldEvent = Event.Data.Get<FSubmissionEvent::FYieldSyncPoint>();
			Builder.AddNameValueTextLine(EventLabel, FString::Printf(TEXT("payload=0x%llx syncpoint=0x%llx"), YieldEvent.PayloadID, YieldEvent.SyncPointID));
			break;
		}
		case ESubmissionEvent_YieldManualFence:
		{
			const FSubmissionEvent::FYieldManualFence& YieldEvent = Event.Data.Get<FSubmissionEvent::FYieldManualFence>();
			Builder.AddNameValueTextLine(EventLabel, FString::Printf(TEXT("payload=0x%llx fence=0x%llx value=%llu"), YieldEvent.PayloadID, YieldEvent.FenceID, YieldEvent.FenceValue));
			break;
		}
		case ESubmissionEvent_WaitQueueFence:
		{
			const FSubmissionEvent::FWaitQueueFence& WaitQueueFenceEvent = Event.Data.Get<FSubmissionEvent::FWaitQueueFence>();
			Builder.AddNameValueTextLine(EventLabel, FString::Printf(TEXT("wait on %s value=%llu payload=0x%llx"), GetPipeName(WaitQueueFenceEvent.OtherQueue), WaitQueueFenceEvent.Value, WaitQueueFenceEvent.PayloadID));
			break;
		}
		case ESubmissionEvent_WaitManualFence:
		{
			const FSubmissionEvent::FWaitManualFence& WaitManualFenceEvent = Event.Data.Get<FSubmissionEvent::FWaitManualFence>();
			Builder.AddNameValueTextLine(EventLabel, FString::Printf(TEXT("fence=0x%llx value=%llu payload=0x%llx"), WaitManualFenceEvent.FenceID, WaitManualFenceEvent.Value, WaitManualFenceEvent.PayloadID));
			break;
		}
		case ESubmissionEvent_Execute:
		{
			const FSubmissionEvent::FExecute& ExecuteEvent = Event.Data.Get<FSubmissionEvent::FExecute>();
			Builder.AddNameValueTextLine(EventLabel, TEXTVIEW(""));
			{
				Builder.AddNameValueTextLine(TEXTVIEW("Payloads:"), LexToString(ExecuteEvent.PayloadIDs.Num()));
				FAppIDListBuilder<uint64> ListBuilder(Builder, ExecuteEvent.PayloadIDs.Num());
				for (uint32 PayloadID : ExecuteEvent.PayloadIDs) { ListBuilder.AddItem(Provider.GetPlatformPayload(PayloadID).AppID); }
				ListBuilder.Finish();
			}
			{
				Builder.AddNameValueTextLine(TEXTVIEW("Platform CmdLists:"), LexToString(ExecuteEvent.PlatformCmdListIDs.Num()));
				FAppIDListBuilder<uint64> ListBuilder(Builder, ExecuteEvent.PlatformCmdListIDs.Num());
				ListBuilder.AddIDs(ExecuteEvent.PlatformCmdListIDs, true);
			}
			break;
		}
		case ESubmissionEvent_SignalManualFence:
		{
			const FSubmissionEvent::FSignalManualFence& SignalManualFenceEvent = Event.Data.Get<FSubmissionEvent::FSignalManualFence>();
			Builder.AddNameValueTextLine(EventLabel, FString::Printf(TEXT("fence=0x%llx value=%llu payload=0x%llx"), SignalManualFenceEvent.FenceID, SignalManualFenceEvent.Value, SignalManualFenceEvent.PayloadID));
			break;
		}
		case ESubmissionEvent_SignalQueueFence:
		{
			const FSubmissionEvent::FSignalQueueFence& SignalQueueFenceEvent = Event.Data.Get<FSubmissionEvent::FSignalQueueFence>();
			Builder.AddNameValueTextLine(EventLabel, FString::Printf(TEXT("value=%llu payload=0x%llx"), SignalQueueFenceEvent.Value, SignalQueueFenceEvent.PayloadID));
			break;
		}
		case ESubmissionEvent_ResolveSyncPoint:
		{
			const FSubmissionEvent::FResolveSyncPoint& ResolveSyncPointEvent = Event.Data.Get<FSubmissionEvent::FResolveSyncPoint>();
			Builder.AddNameValueTextLine(EventLabel, FString::Printf(TEXT("payload=0x%llx syncpoint=0x%llx value=%llu"), Provider.GetPlatformPayload(ResolveSyncPointEvent.PayloadID).AppID, Provider.GetSyncPoint(ResolveSyncPointEvent.SyncPointID).AppID, ResolveSyncPointEvent.Value));
			break;
		}
		}
	}
}

void FSubmissionQueueTrack::PopulateEventDetails(IEventDetailsBuilder& Builder, const FTimingEvent& Event) const
{
	const uint32 Depth = Event.GetDepth();
	const uint32 ParentItemID = static_cast<uint32>(Event.GetType() >> 32);
	const uint32 ItemID = static_cast<uint32>(Event.GetType());

	if (Depth == 0)
	{
		PopulateSubmissionBatchDetails(Builder, ItemID);
		return;
	}

	const IRenderTraceProvider& Provider = Builder.GetProvider();
	const FSubmissionBatch& Batch = Provider.GetSubmissionBatch(ParentItemID);
	const FSubmissionEvent& SubmissionEvent = Batch.Events[ItemID];
	switch (SubmissionEvent.Data.GetIndex())
	{
	case ESubmissionEvent_YieldSyncPoint:
	{
		const FSubmissionEvent::FYieldSyncPoint& YieldEvent = SubmissionEvent.Data.Get<FSubmissionEvent::FYieldSyncPoint>();
		Builder.AddTitle(TEXTVIEW("Yield SyncPoint"));
		Builder.AddNameValueTextLine(TEXTVIEW("Queue:"), GetPipeName(SubmissionEvent.Queue));
		Builder.AddNameValueTextLine(TEXTVIEW("Pending payload:"), FString::Printf(TEXT("0x%llx"), YieldEvent.PayloadID));
		Builder.AddNameValueTextLine(TEXTVIEW("Pending sync point:"), FString::Printf(TEXT("0x%llx"), YieldEvent.SyncPointID));
		break;
	}

	case ESubmissionEvent_YieldManualFence:
	{
		const FSubmissionEvent::FYieldManualFence& YieldEvent = SubmissionEvent.Data.Get<FSubmissionEvent::FYieldManualFence>();
		Builder.AddTitle(TEXTVIEW("Yield Manual Fence"));
		Builder.AddNameValueTextLine(TEXTVIEW("Queue:"), GetPipeName(SubmissionEvent.Queue));
		Builder.AddNameValueTextLine(TEXTVIEW("Pending payload:"), FString::Printf(TEXT("0x%llx"), YieldEvent.PayloadID));
		Builder.AddNameValueTextLine(TEXTVIEW("Pending fence:"), FString::Printf(TEXT("0x%llx"), YieldEvent.FenceID));
		Builder.AddNameValueTextLine(TEXTVIEW("Fence value:"), FString::Printf(TEXT("%llu"), YieldEvent.FenceValue));
		break;
	}

	case ESubmissionEvent_WaitQueueFence:
	{
		const FSubmissionEvent::FWaitQueueFence& WaitQueueFenceEvent = SubmissionEvent.Data.Get<FSubmissionEvent::FWaitQueueFence>();
		Builder.AddTitle(TEXTVIEW("Wait Queue Fence"));
		Builder.AddNameValueTextLine(TEXTVIEW("Queue:"), GetPipeName(SubmissionEvent.Queue));
		Builder.AddNameValueTextLine(TEXTVIEW("Other Queue:"), GetPipeName(WaitQueueFenceEvent.OtherQueue));
		Builder.AddNameValueTextLine(TEXTVIEW("Fence Value:"), FString::Printf(TEXT("%llu"), WaitQueueFenceEvent.Value));
		Builder.AddNameValueTextLine(TEXTVIEW("Payload:"), FString::Printf(TEXT("0x%llx"), WaitQueueFenceEvent.PayloadID));
		break;
	}

	case ESubmissionEvent_WaitManualFence:
	{
		const FSubmissionEvent::FWaitManualFence& WaitManualFenceEvent = SubmissionEvent.Data.Get<FSubmissionEvent::FWaitManualFence>();
		Builder.AddTitle(TEXTVIEW("Wait Manual Fence"));
		Builder.AddNameValueTextLine(TEXTVIEW("Queue:"), GetPipeName(SubmissionEvent.Queue));
		Builder.AddNameValueTextLine(TEXTVIEW("Fence ID:"), FString::Printf(TEXT("0x%llx"), WaitManualFenceEvent.FenceID));
		Builder.AddNameValueTextLine(TEXTVIEW("Fence Value:"), FString::Printf(TEXT("%llu"), WaitManualFenceEvent.Value));
		Builder.AddNameValueTextLine(TEXTVIEW("Payload:"), FString::Printf(TEXT("0x%llx"), WaitManualFenceEvent.PayloadID));
		break;
	}

	case ESubmissionEvent_Execute:
	{
		const FSubmissionEvent::FExecute& ExecuteEvent = SubmissionEvent.Data.Get<FSubmissionEvent::FExecute>();
		Builder.AddTitle(TEXTVIEW("Execute Command Lists"));
		Builder.AddNameValueTextLine(TEXTVIEW("Queue:"), GetPipeName(SubmissionEvent.Queue));

		Builder.AddNameValueTextLine(TEXTVIEW("Payloads:"), LexToString(ExecuteEvent.PayloadIDs.Num()));
		{
			FAppIDListBuilder<uint64> ListBuilder(Builder, ExecuteEvent.PayloadIDs.Num());
			for (uint32 PayloadID : ExecuteEvent.PayloadIDs) { ListBuilder.AddItem(Provider.GetPlatformPayload(PayloadID).AppID); }
			ListBuilder.Finish();
		}

		Builder.AddNameValueTextLine(TEXTVIEW("Command Lists:"), LexToString(ExecuteEvent.PlatformCmdListIDs.Num()));
		{
			FAppIDListBuilder<uint64> ListBuilder(Builder, ExecuteEvent.PlatformCmdListIDs.Num());
			ListBuilder.AddIDs(ExecuteEvent.PlatformCmdListIDs, true);
		}
		break;
	}

	case ESubmissionEvent_SignalManualFence:
	{
		const FSubmissionEvent::FSignalManualFence& SignalManualFenceEvent = SubmissionEvent.Data.Get<FSubmissionEvent::FSignalManualFence>();
		Builder.AddTitle(TEXTVIEW("Signal Manual Fence"));
		Builder.AddNameValueTextLine(TEXTVIEW("Queue:"), GetPipeName(SubmissionEvent.Queue));
		Builder.AddNameValueTextLine(TEXTVIEW("Fence ID:"), FString::Printf(TEXT("0x%llx"), SignalManualFenceEvent.FenceID));
		Builder.AddNameValueTextLine(TEXTVIEW("Fence Value:"), FString::Printf(TEXT("%llu"), SignalManualFenceEvent.Value));
		Builder.AddNameValueTextLine(TEXTVIEW("Payload:"), FString::Printf(TEXT("0x%llx"), SignalManualFenceEvent.PayloadID));
		break;
	}

	case ESubmissionEvent_SignalQueueFence:
	{
		const FSubmissionEvent::FSignalQueueFence& SignalQueueFenceEvent = SubmissionEvent.Data.Get<FSubmissionEvent::FSignalQueueFence>();
		Builder.AddTitle(TEXTVIEW("Signal Queue Fence"));
		Builder.AddNameValueTextLine(TEXTVIEW("Queue:"), GetPipeName(SubmissionEvent.Queue));
		Builder.AddNameValueTextLine(TEXTVIEW("Fence Value:"), FString::Printf(TEXT("%llu"), SignalQueueFenceEvent.Value));
		Builder.AddNameValueTextLine(TEXTVIEW("Payload:"), FString::Printf(TEXT("0x%llx"), SignalQueueFenceEvent.PayloadID));
		break;
	}

	case ESubmissionEvent_ResolveSyncPoint:
	{
		const FSubmissionEvent::FResolveSyncPoint& ResolveSyncPointEvent = SubmissionEvent.Data.Get<FSubmissionEvent::FResolveSyncPoint>();
		Builder.AddTitle(TEXTVIEW("Resolve Sync Point"));
		Builder.AddNameValueTextLine(TEXTVIEW("Queue:"), GetPipeName(SubmissionEvent.Queue));
		Builder.AddNameValueTextLine(TEXTVIEW("Payload:"), FString::Printf(TEXT("0x%llx"), Provider.GetPlatformPayload(ResolveSyncPointEvent.PayloadID).AppID));
		Builder.AddNameValueTextLine(TEXTVIEW("Sync Point:"), FString::Printf(TEXT("0x%llx"), Provider.GetSyncPoint(ResolveSyncPointEvent.SyncPointID).AppID));
		Builder.AddNameValueTextLine(TEXTVIEW("Fence Value:"), FString::Printf(TEXT("%llu"), ResolveSyncPointEvent.Value));
		break;
	}
	}
}

FInterruptTrack::FInterruptTrack(const FRenderTraceTimingViewSession& InSharedData, const FString& Name, const IRenderTraceProvider::TEventTimeline* InTimeline)
	: FRenderTraceTrack(InSharedData, Name, InTimeline)
{
}

void FInterruptTrack::OnRenderTraceEventSelected(const FTimingEvent& InSelectedEvent) const
{
	UE::Insights::Timing::ITimingViewSession* TimingView = SharedData.GetTimingViewSession();
	const IRenderTraceProvider* Provider = SharedData.GetRenderTraceProvider();
	if (!TimingView || !Provider)
	{
		return;
	}

	const uint32 Depth = InSelectedEvent.GetDepth();
	const uint32 ParentItemID = static_cast<uint32>(InSelectedEvent.GetType() >> 32);
	const uint32 ItemID = static_cast<uint32>(InSelectedEvent.GetType());

	switch (Depth)
	{
	case 0:
	{
		// Wakeup event. Add an arrow to the CPU thread where batch was processed.
		const FInterruptWakeUp& WakeUp = Provider->GetInterruptWakeUp(ItemID);

		TSharedPtr<const FBaseTimingTrack> CpuThreadTrack = SharedData.GetCpuThreadTrack(WakeUp.ThreadID);
		if (CpuThreadTrack)
		{
			TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
				SharedThis(this), WakeUp.StartTime, 0, CpuThreadTrack, WakeUp.StartTime, 0, GRelationColorExecThread
			);
			TimingView->AddRelation(Relation);
		}

		return;
	}

	case 1:
	{
		// Signal subevent. Add an arrow from the payload on the translate track.
		const FInterruptWakeUp& WakeUp = Provider->GetInterruptWakeUp(ParentItemID);
		const FInterruptFenceSignaledEvent& SignalEvent = WakeUp.SignalEvents[ItemID];

		const FPlatformPayload& Payload = Provider->GetPlatformPayload(SignalEvent.PayloadID);
		if (Payload.TranslateEvent.ParentItemID == INVALID_EVENT_ID)
		{
			return;
		}

		const FRHITranslateTask& TranslateTask = Provider->GetRHITranslateTask(Payload.TranslateEvent.ParentItemID);
		check(TranslateTask.Type == ERHITranslateTaskType::Translate);

		TSharedPtr<const FRHIThreadTrack> TranslateTrack = SharedData.GetRHIThreadTrack(TranslateTask.ThreadID);
		if (TranslateTrack.IsValid())
		{
			const uint32 FirstContextDepth = 2;
			const uint32 SourceDepth = FirstContextDepth + 2 * Payload.PipeIdx + 1;

			TUniquePtr<ITimingEventRelation> Relation = MakeUnique<FRenderTraceRelation>(
				TranslateTrack, Payload.EndTime, SourceDepth, SharedThis(this), InSelectedEvent.GetStartTime(), 1, GRelationColorInterrupt
			);

			TimingView->AddRelation(Relation);
		}

		return;
	}
	}
}

FString FInterruptTrack::GetEventName(const IRenderTraceProvider* Provider, uint32 InDepth, const FRenderTraceEvent& InEvent) const
{
	if (InDepth == 0)
	{
		return TEXT("ProcessInterruptQueue");
	}

	const FInterruptWakeUp& WakeUp = Provider->GetInterruptWakeUp(InEvent.ParentItemID);
	const FInterruptFenceSignaledEvent& SignalEvent = WakeUp.SignalEvents[InEvent.ItemID];
	return FString::Printf(TEXT("%s Signaled %llu"), GetPipeName(SignalEvent.Queue), SignalEvent.FenceValue);
}

void FInterruptTrack::PopulateEventDetails(IEventDetailsBuilder& Builder, const FTimingEvent& Event) const
{
	const IRenderTraceProvider& Provider = Builder.GetProvider();

	const uint32 Depth = Event.GetDepth();
	const uint32 ParentItemID = static_cast<uint32>(Event.GetType() >> 32);
	const uint32 ItemID = static_cast<uint32>(Event.GetType());

	switch (Depth)
	{
	case 0:
	{
		const FInterruptWakeUp& WakeUp = Provider.GetInterruptWakeUp(ItemID);
		Builder.AddTitle(TEXTVIEW("ProcessInterruptQueue"));
		Builder.AddNameValueTextLine(TEXTVIEW("Exit Status:"), GetExitStatusStr(WakeUp.ExitStatus));
		Builder.AddNameValueTextLine(TEXTVIEW("Signal Events:"), LexToString(WakeUp.SignalEvents.Num()));

		if (Builder.GetDetailLevel() == EEventDetailLevel::Full)
		{
			for (const FInterruptFenceSignaledEvent& SignalEvent : WakeUp.SignalEvents)
			{
				const FPlatformPayload& Payload = Provider.GetPlatformPayload(SignalEvent.PayloadID);
				FString Label = FString::Printf(TEXT("[%s] Signaled:"), GetPipeName(SignalEvent.Queue));
				FString Details = FString::Printf(TEXT("value=%llu payload=0x%llx lastCPU=%llu"), SignalEvent.FenceValue, Payload.AppID, SignalEvent.LastCPUSignaledFenceValue);
				Builder.AddNameValueTextLine(Label, Details);
			}
		}

		return;
	}

	case 1:
	{
		const FInterruptWakeUp& WakeUp = Provider.GetInterruptWakeUp(ParentItemID);
		const FInterruptFenceSignaledEvent& SignalEvent = WakeUp.SignalEvents[ItemID];
		const FPlatformPayload& Payload = Provider.GetPlatformPayload(SignalEvent.PayloadID);

		Builder.AddTitle(TEXTVIEW("Signal Queue Fence"));
		Builder.AddNameValueTextLine(TEXTVIEW("Queue:"), GetPipeName(SignalEvent.Queue));
		Builder.AddNameValueTextLine(TEXTVIEW("Payload:"), FString::Printf(TEXT("0x%llx"), Payload.AppID));
		Builder.AddNameValueTextLine(TEXTVIEW("Fence Value:"), LexToString(SignalEvent.FenceValue));
		Builder.AddNameValueTextLine(TEXTVIEW("Last CPU Signaled Value:"), LexToString(SignalEvent.LastCPUSignaledFenceValue));
		return;
	}
	}
}

} //namespace RenderTraceInsights
} //namespace UE
