// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/ViewModels/TimingEventsTrack.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "Insights/ViewModels/ITimingEvent.h"
#include "IRenderTraceProvider.h"

class FMenuBuilder;
class ITimingTrackDrawContext;
class ITimingTrackUpdateContext;
class FTimingTrackViewport;
class FTimingEvent;

namespace UE {
namespace RenderTraceInsights {

class FRenderTraceTimingViewSession;
class IEventDetailsBuilder;

class FRenderTraceRelation : public ITimingEventRelation
{
	INSIGHTS_DECLARE_RTTI(FRenderTraceRelation, ITimingEventRelation)
public:
	UE_NONCOPYABLE(FRenderTraceRelation)

	FRenderTraceRelation(TSharedPtr<const FBaseTimingTrack> InSourceTrack, double InSourceTime, uint32 InSourceDepth, TSharedPtr<const FBaseTimingTrack> InTargetTrack, double InTargetTime, uint32 InTargetDepth, const FLinearColor& InColor);
	virtual ~FRenderTraceRelation() = default;

	void Draw(const UE::Insights::FDrawContext& DrawContext, const FTimingTrackViewport& Viewport, const ITimingViewDrawHelper& Helper, const ITimingEventRelation::EDrawFilter Filter) override;

private:
	TWeakPtr<const FBaseTimingTrack> SourceTrack;
	TWeakPtr<const FBaseTimingTrack> TargetTrack;
	double SourceTime;
	double TargetTime;
	uint32 SourceDepth;
	uint32 TargetDepth;
	FLinearColor Color;
};

class FRenderTraceTrack : public FTimingEventsTrack
{
	UE_NONCOPYABLE(FRenderTraceTrack)
	INSIGHTS_DECLARE_RTTI(FRenderTraceTrack, FTimingEventsTrack)
public:
	FRenderTraceTrack(const FRenderTraceTimingViewSession& InSharedData, const FString& Name, const IRenderTraceProvider::TEventTimeline* InTimeline);

	// FBaseTimingTrack interface.
	const TSharedPtr<const ITimingEvent> GetEvent(double InTime, double SecondsPerPixel, int32 Depth) const override;
	void InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& TooltipEvent) const override;
	void OnClipboardCopyEvent(const ITimingEvent& InSelectedEvent) const override;

	// FTimingEventsTrack interface.
	void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;

	// Interface that subclasses must implement.
	virtual void OnRenderTraceEventSelected(const FTimingEvent& InSelectedEvent) const {}
	virtual FString GetEventName(const IRenderTraceProvider* Provider, uint32 InDepth, const FRenderTraceEvent& InEvent) const { return TEXT(""); }
	virtual void PopulateEventDetails(IEventDetailsBuilder& Builder, const FTimingEvent& Event) const {}
	virtual void CustomizeEventDrawState(ITimingEventsTrackDrawStateBuilder& Builder, double StartTime, double EndTime, uint32 Depth, const FRenderTraceEvent& Event) const {}
	virtual TSharedPtr<FTimingEvent> GetEventAtTime(double EventTime, int32 EventDepth, double TimelineEventStart, double TimelineEventEnd, int32 TimelineEventDepth, const FRenderTraceEvent& TimelineEvent) const { return nullptr; }

protected:
	const FRenderTraceTimingViewSession& SharedData;
	const IRenderTraceProvider::TEventTimeline* Timeline;
};

class FCommandListTrack : public FRenderTraceTrack
{
	UE_NONCOPYABLE(FCommandListTrack)
	INSIGHTS_DECLARE_RTTI(FCommandListTrack, FRenderTraceTrack)
public:
	FCommandListTrack(const FRenderTraceTimingViewSession& InSharedData, const FString& Name, const IRenderTraceProvider::TEventTimeline* InTimeline);

	// FRenderTraceTrack interface.
	void OnRenderTraceEventSelected(const FTimingEvent& InSelectedEvent) const override;
	FString GetEventName(const IRenderTraceProvider* Provider, uint32 InDepth, const FRenderTraceEvent& InEvent) const override;
	void PopulateEventDetails(IEventDetailsBuilder& Builder, const FTimingEvent& Event) const override;

private:
	void OnCommandListSelected(const FTimingEvent& InSelectedEvent) const;
	void OnRecordingEventSelected(const FTimingEvent& InSelectedEvent) const;
	void PopulateCommandListEventDetails(IEventDetailsBuilder& Builder, uint32 CmdListID) const;
	void PopulateRecordingEventDetails(IEventDetailsBuilder& Builder, uint32 PassID, double StartTime, double EndTime) const;
};

class FRDGThreadTrack : public FRenderTraceTrack
{
	UE_NONCOPYABLE(FRDGThreadTrack)
	INSIGHTS_DECLARE_RTTI(FRDGThreadTrack, FRenderTraceTrack)
public:
	FRDGThreadTrack(const FRenderTraceTimingViewSession& InSharedData, const FString& Name, const IRenderTraceProvider::TEventTimeline* InTimeline);

	// FRenderTraceTrack interface.
	void OnRenderTraceEventSelected(const FTimingEvent& InSelectedEvent) const override;
	FString GetEventName(const IRenderTraceProvider* Provider, uint32 InDepth, const FRenderTraceEvent& InEvent) const override;
	void PopulateEventDetails(IEventDetailsBuilder& Builder, const FTimingEvent& Event) const override;
};

class FRenderThreadSubmissionTrack : public FRenderTraceTrack
{
	UE_NONCOPYABLE(FRenderThreadSubmissionTrack)
	INSIGHTS_DECLARE_RTTI(FRenderThreadSubmissionTrack, FRenderTraceTrack)
public:
	FRenderThreadSubmissionTrack(const FRenderTraceTimingViewSession& InSharedData, const FString& Name, const IRenderTraceProvider::TEventTimeline* InTimeline);

	// FRenderTraceTrack interface.
	void OnRenderTraceEventSelected(const FTimingEvent& InSelectedEvent) const override;
	FString GetEventName(const IRenderTraceProvider* Provider, uint32 InDepth, const FRenderTraceEvent& InEvent) const override;
	void PopulateEventDetails(IEventDetailsBuilder& Builder, const FTimingEvent& Event) const override;
};

class FRHIThreadTrack : public FRenderTraceTrack
{
	UE_NONCOPYABLE(FRHIThreadTrack)
	INSIGHTS_DECLARE_RTTI(FRHIThreadTrack, FRenderTraceTrack)
public:
	FRHIThreadTrack(const FRenderTraceTimingViewSession& InSharedData, const FString& Name, const IRenderTraceProvider::TEventTimeline* InTimeline);

	// FRenderTraceTrack interface.
	void OnRenderTraceEventSelected(const FTimingEvent& InSelectedEvent) const override;
	FString GetEventName(const IRenderTraceProvider* Provider, uint32 InDepth, const FRenderTraceEvent& InEvent) const override;
	void PopulateEventDetails(IEventDetailsBuilder& Builder, const FTimingEvent& Event) const override;
	void CustomizeEventDrawState(ITimingEventsTrackDrawStateBuilder& Builder, double StartTime, double EndTime, uint32 Depth, const FRenderTraceEvent& Event) const override;
	TSharedPtr<FTimingEvent> GetEventAtTime(double EventTime, int32 EventDepth, double TimelineEventStart, double TimelineEventEnd, int32 TimelineEventDepth, const FRenderTraceEvent& TimelineEvent) const override;

private:
	void CacheSplitFromTask(const IRenderTraceProvider& Provider, uint32 TaskID) const;
	void CacheSplitToTask(const IRenderTraceProvider& Provider, uint32 TaskID) const;
	void OnTranslateEventSelected(const FTimingEvent& InSelectedEvent) const;
	void OnTranslateCommandListEventSelected(const FTimingEvent& InSelectedEvent) const;
	void OnPayloadEventSelected(const FTimingEvent& InSelectedEvent, uint32 TranslateJobID, uint8 PipeIdx, uint32 PayloadIdx) const;
	void PopulateTranslateJobDetails(IEventDetailsBuilder& Builder, uint32 TaskID, const FRHITranslateTask& Task) const;
	void ListSyncPoints(IEventDetailsBuilder& Builder, const TCHAR* Title, TConstArrayView<uint32> SyncPointIDs) const;
	void PopulatePayloadDetails(IEventDetailsBuilder& Builder, uint32 TranslateJobID, uint32 PipeIdx, uint32 PayloadIdx) const;
	void PopulateTranslateContextDetails(IEventDetailsBuilder& Builder, uint32 TranslateJobID, uint32 PipeIdx) const;
};

class FRHISubmissionTrack : public FRenderTraceTrack
{
	UE_NONCOPYABLE(FRHISubmissionTrack)
	INSIGHTS_DECLARE_RTTI(FRHISubmissionTrack, FRenderTraceTrack)
public:
	FRHISubmissionTrack(const FRenderTraceTimingViewSession& InSharedData, const FString& Name, const IRenderTraceProvider::TEventTimeline* InTimeline);

	// FRenderTraceTrack interface.
	void OnRenderTraceEventSelected(const FTimingEvent& InSelectedEvent) const override;
	FString GetEventName(const IRenderTraceProvider* Provider, uint32 InDepth, const FRenderTraceEvent& InEvent) const override;
	void PopulateEventDetails(IEventDetailsBuilder& Builder, const FTimingEvent& Event) const override;

private:
	void OnSubmitEventSelected(const FTimingEvent& InSelectedEvent) const;
	void OnSubmitTranslateJobEventSelected(const FTimingEvent& InSelectedEvent) const;
	void PopulateSubmitTaskDetails(IEventDetailsBuilder& Builder, uint32 TaskID, const FRHITranslateTask& Task) const;
};

class FSubmissionQueueTrack : public FRenderTraceTrack
{
	UE_NONCOPYABLE(FSubmissionQueueTrack)
	INSIGHTS_DECLARE_RTTI(FSubmissionQueueTrack, FRenderTraceTrack)
public:
	FSubmissionQueueTrack(const FRenderTraceTimingViewSession& InSharedData, const FString& Name, const IRenderTraceProvider::TEventTimeline* InTimeline);

	// FRenderTraceTrack interface.
	void OnRenderTraceEventSelected(const FTimingEvent& InSelectedEvent) const override;
	FString GetEventName(const IRenderTraceProvider* Provider, uint32 InDepth, const FRenderTraceEvent& InEvent) const override;
	void PopulateEventDetails(IEventDetailsBuilder& Builder, const FTimingEvent& Event) const override;

private:
	void PopulateSubmissionBatchDetails(IEventDetailsBuilder& Builder, uint32 ItemID) const;
	void OnSubmissionBatchSelected(const FTimingEvent& InSelectedEvent, uint32 BatchID) const;
	void OnResolveSyncPointEventSelected(const FTimingEvent& InSelectedEvent, const FSubmissionEvent::FResolveSyncPoint& ResolveEvent) const;
	void OnExecuteEventSelected(const FTimingEvent& InSelectedEvent, const FSubmissionEvent::FExecute& ExecEvent) const;
};

class FInterruptTrack : public FRenderTraceTrack
{
	UE_NONCOPYABLE(FInterruptTrack)
	INSIGHTS_DECLARE_RTTI(FInterruptTrack, FRenderTraceTrack)
public:
	FInterruptTrack(const FRenderTraceTimingViewSession& InSharedData, const FString& Name, const IRenderTraceProvider::TEventTimeline* InTimeline);

	// FRenderTraceTrack interface.
	void OnRenderTraceEventSelected(const FTimingEvent& InSelectedEvent) const override;
	FString GetEventName(const IRenderTraceProvider* Provider, uint32 InDepth, const FRenderTraceEvent& InEvent) const override;
	void PopulateEventDetails(IEventDetailsBuilder& Builder, const FTimingEvent& Event) const override;
};

} // namespace RenderTraceInsights
} //namespace UE
