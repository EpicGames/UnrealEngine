// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/Array.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/TVariant.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "Model/MonotonicTimeline.h"
#include "Templates/Function.h"

namespace TraceServices { class IAnalysisSession; }

namespace UE
{
namespace RenderTraceInsights
{

static constexpr uint32 INVALID_EVENT_ID = ~0u;

struct FTimeInterval
{
	FTimeInterval() : Start(0.0), End(0.0) {}
	explicit FTimeInterval(double InStart) : Start(InStart), End(0.0) {}
	FTimeInterval(double InStart, double InEnd) : Start(InStart), End(InEnd) {}
	double Start;
	double End;
};

struct FEventInterval
{
	FEventInterval(uint32 InItemID, double InStartTime) : ItemID(InItemID), Start(InStartTime), End(0.0) {}
	FEventInterval(uint32 InItemID, double InStartTime, double InEndTime) : ItemID(InItemID), Start(InStartTime), End(InEndTime) {}
	uint32 ItemID;
	double Start;
	double End;
};

struct FRenderTraceEvent
{
	FRenderTraceEvent() : ItemID(INVALID_EVENT_ID), ParentItemID(INVALID_EVENT_ID) {}
	explicit FRenderTraceEvent(uint32 InItemID, uint32 InParentItemID = INVALID_EVENT_ID) : ItemID(InItemID), ParentItemID(InParentItemID) {}

	uint32 ItemID;
	uint32 ParentItemID;
};

enum class ECommandListType : uint8
{
	Regular,
	Immediate,
	Detached
};

enum ECommandListRecordingFlags
{
	ECommandListRecordingFlag_UsesLockFence = 1 << 0
};

enum ERHITranslateJobSplitFlags
{
	ERHITranslateJobSplitFlag_Parallel = 1 << 0,
	ERHITranslateJobSplitFlag_Threshold = 1 << 1,
	ERHITranslateJobSplitFlag_ParentChild = 1 << 2,
	ERHITranslateJobSplitFlag_JumpThreads = 1 << 3
};

struct FCommandListInstance
{
	UE_NONCOPYABLE(FCommandListInstance)
	FCommandListInstance(uint64 InAppID, double InCreateTime, ECommandListType InType)
		: AppID(InAppID)
		, Type(InType)
		, CreateTime(InCreateTime)
	{}

	uint64 AppID;
	ECommandListType Type;

	// Important timestamps for the lifetime of the command list.
	double CreateTime;
	double FinishRecordingTime{};
	double SubmitTime{};
	uint32 SubmitPassID = INVALID_EVENT_ID;
	double DestroyTime{};

	// Recording information.
	TArray<FEventInterval, TInlineAllocator<8>> RecordingEvents;
	uint8 RecordingFlags = 0; // ECommandListRecordingFlags

	// Detach information.
	uint32 DetachedCmdListID = INVALID_EVENT_ID;
	double DetachTime{};
	uint32 SourceCmdListID = INVALID_EVENT_ID;

	// Dispatch and translate information.
	double DispatchTime{};
	uint32 TranslateTaskID = INVALID_EVENT_ID;
	uint64 TranslatePrevJobAppID = 0;
	uint8 TranslateSplitFlags = 0; // ERHITranslateJobSplitFlags

	// Timeline information.
	double TimelineStart{};
	int32 TimelineIndex = -1;
};

enum class ERDGPassType : uint8
{
	Regular,
	Dispatch,
	Submit
};

// Must be kept in sync with the enum of the same name in RenderGraphDefinitions.h.
enum ERDGPassFlags
{
	ERDGPassFlag_Raster = 1 << 0,
	ERDGPassFlag_Compute = 1 << 1,
	ERDGPassFlag_AsyncCompute = 1 << 2,
	ERDGPassFlag_Copy = 1 << 3,
	ERDGPassFlag_NeverCull = 1 << 4,
	ERDGPassFlag_SkipRenderPass = 1 << 5,
	ERDGPassFlag_NeverMerge = 1 << 6,
	ERDGPassFlag_NeverParallel = 1 << 7
};

// Must be kept in sync with the enum of the same name in RenderGraphPass.h.
enum class ERDGPassTaskMode : uint8
{
	Inline = 0,
	Await = 1,
	Async = 2,
	NotSet = 0xff
};

// Must be kept in sync ERHISubmitFlags. We don't want to use the enum in RHICommandList.h directly because some values are conditionally compiled.
enum ERDGPassSubmitFlags
{
	ERDGPassSubmitFlag_SubmitToGPU = 1 << 0,
	ERDGPassSubmitFlag_DeleteResources = 1 << 1,
	ERDGPassSubmitFlag_FlushRHIThread = 1 << 2,
	ERDGPassSubmitFlag_EndFrame = 1 << 3,
	ERDGPassSubmitFlag_EnableBypass = 1 << 4,
	ERDGPassSubmitFlag_DisableBypass = 1 << 5,
	ERDGPassSubmitFlag_EnableDrawEvents = 1 << 6,
	ERDGPassSubmitFlag_DisableDrawEvents = 1 << 7
};

struct FRDGPassInstance
{
	UE_NONCOPYABLE(FRDGPassInstance)
	FRDGPassInstance(const TCHAR* InName, uint32 InExecThreadID, double InStartTime, ERDGPassType InType)
		: Name(InName)
		, Type(InType)
		, StartTime(InStartTime)
		, ExecThreadID(InExecThreadID)
	{}

	// Pass names are allocated from the session's linear allocator using the IAnalysisSession::StoreString,
	// so we can just point to them because the lifetime of the pass instances is tied to the lifetime of
	// the session.
	const TCHAR* Name;
	ERDGPassType Type;
	uint16 Flags = 0; // Either ERDGPassFlags or ERDGPassSubmitFlags.
	ERDGPassTaskMode TaskMode = ERDGPassTaskMode::NotSet;

	// Execution information.
	double StartTime;
	double EndTime{};
	uint32 ExecThreadID;

	// Recording command lists.
	TArray<uint32, TInlineAllocator<8>> CommandListIDs;

	// Extra data for render thread submit events.
	uint32 SubmitTaskID = INVALID_EVENT_ID;
};

enum class ESyncPointType : uint8
{
	GPU = 0,
	GPUAndCPU = 1,
	Manual = 2
};

struct FSyncPoint
{
	UE_NONCOPYABLE(FSyncPoint)
	FSyncPoint(uint64 InAppID, ESyncPointType InType) : AppID(InAppID), Type(InType) {}
	uint64 AppID;
	uint64 ResolvedValue = 0;
	ESyncPointType Type;
	uint32 ResolvedByPayload = INVALID_EVENT_ID;
};

struct FPlatformPayload
{
	UE_NONCOPYABLE(FPlatformPayload)
	explicit FPlatformPayload(uint64 InAppID, double InStartTime, uint8 InPipeIdx) : AppID(InAppID), StartTime(InStartTime), PipeIdx(InPipeIdx) {}
	uint64 AppID;
	double StartTime;
	double EndTime = 0.0;
	uint8 PipeIdx;
	TArray<uint32, TInlineAllocator<1>> WaitSyncPoints;
	TArray<uint64, TInlineAllocator<1>> CmdLists;
	TArray<uint32, TInlineAllocator<1>> SignalSyncPoints;
	FRenderTraceEvent TranslateEvent;
	FRenderTraceEvent ExecutionEvent;
	FRenderTraceEvent InterruptEvent;
	TArray<FRenderTraceEvent> ResolveSyncPointEvents;
};

struct FRHITranslateContext
{
	UE_NONCOPYABLE(FRHITranslateContext)
	FRHITranslateContext() {}
	uint64 RHIContextID = 0;
	TArray<FTimeInterval, TInlineAllocator<4>> ActiveIntervals;
	TArray<uint32, TInlineAllocator<2>> PayloadIDs;
};

enum class ERHITranslateTaskType : uint8
{
	Translate,
	Submit
};

enum ERHITranslateJobFlags
{
	ERHITranslateJobFlag_Parallel = 1 << 0,
	ERHITranslateJobFlag_UsingSubCmdLists = 1 << 1
};

struct FRHITranslateTask
{
	UE_NONCOPYABLE(FRHITranslateTask)

	FRHITranslateTask(uint64 InAppID, ERHITranslateTaskType InType, uint32 InThreadID, double InStartTime)
		: AppID(InAppID)
		, Type(InType)
		, ThreadID(InThreadID)
		, StartTime(InStartTime)
	{}

	// Information about the task.
	uint64 AppID;
	ERHITranslateTaskType Type;
	uint32 ThreadID;
	double StartTime;
	double EndTime{};

	// Items processed by the job (command lists, translate jobs etc.).
	TArray<FEventInterval, TInlineAllocator<8>> ProcessedItems;

	// Next task in the translate chain (e.g. dispatch -> translate).
	uint32 NextPhaseTaskID = INVALID_EVENT_ID;

	uint8 JobFlags = 0; // ERHITranslateJobFlags

	// For a submit task, this identifies the render-thread submit pass ID if the instigator is the render thread,
	// or the translate job if it's an eager submit. For a translate tasks, this identifies the submission it
	// triggered, if any.
	uint32 SubmitTaskID = INVALID_EVENT_ID;
	bool bEagerSubmission = false;
	
	// Previous and next jobs when a translate chain is split. This is not computed during analysis because it's too complicated.
	// Instead, we compute and cache these values on demand when an event is selected.
	mutable uint32 SplitFromTranslateTaskID = INVALID_EVENT_ID;
	mutable uint32 SplitToTranslateTaskID = INVALID_EVENT_ID;

	// Translation contexts, one per pipe: direct and async compute.
	FRHITranslateContext Contexts[2];
};

// Must be kept in the same order as the subtype declarations in the FSubmissionEvent::Data variant.
enum ESubmissionEvent
{
	ESubmissionEvent_YieldSyncPoint,
	ESubmissionEvent_YieldManualFence,
	ESubmissionEvent_WaitQueueFence,
	ESubmissionEvent_WaitManualFence,
	ESubmissionEvent_Execute,
	ESubmissionEvent_SignalManualFence,
	ESubmissionEvent_SignalQueueFence,
	ESubmissionEvent_ResolveSyncPoint
};

struct FSubmissionEvent
{
	UE_NONCOPYABLE(FSubmissionEvent)
	FSubmissionEvent(double InTimestamp, uint8 InQueue) : Timestamp(InTimestamp), Queue(InQueue) {}
	~FSubmissionEvent() = default;

	double Timestamp;
	uint8 Queue;

	struct FYieldSyncPoint
	{
		uint64 PayloadID;
		uint64 SyncPointID;
	};

	struct FYieldManualFence
	{
		uint64 PayloadID;
		uint64 FenceID;
		uint64 FenceValue;
	};

	struct FWaitQueueFence
	{
		uint64 PayloadID;
		uint8 OtherQueue;
		uint64 Value;
	};

	struct FWaitManualFence
	{
		uint64 PayloadID;
		uint64 FenceID;
		uint64 Value;
	};

	struct FExecute
	{
		FExecute(TConstArrayView<uint32> InPayloadIDs, TConstArrayView<uint64> InPlatformCmdListIDs) : PayloadIDs(InPayloadIDs), PlatformCmdListIDs(InPlatformCmdListIDs) {}
		TArray<uint32> PayloadIDs;
		TArray<uint64> PlatformCmdListIDs;
	};

	struct FSignalManualFence
	{
		uint64 PayloadID;
		uint64 FenceID;
		uint64 Value;
	};

	struct FSignalQueueFence
	{
		uint64 PayloadID;
		uint64 Value;
	};

	struct FResolveSyncPoint
	{
		uint32 PayloadID;
		uint32 SyncPointID;
		uint64 Value;
	};

	TVariant<FYieldSyncPoint, FYieldManualFence, FWaitQueueFence, FWaitManualFence, FExecute, FSignalManualFence, FSignalQueueFence, FResolveSyncPoint> Data;
};

struct FSubmissionBatch
{
	UE_NONCOPYABLE(FSubmissionBatch)
	FSubmissionBatch(uint32 InThreadID, double InStartTime) : ThreadID(InThreadID), StartTime(InStartTime) {}

	uint32 ThreadID;
	double StartTime;
	double EndTime = 0.0;
	uint8 ExitStatus = 0;
	TArray<FSubmissionEvent> Events;
};

struct FInterruptFenceSignaledEvent
{
	FInterruptFenceSignaledEvent(double InTimestamp, uint8 InQueue, uint32 InPayloadID, uint64 InFenceValue, uint64 InLastCPUSignaledFenceValue) :
		Timestamp(InTimestamp), Queue(InQueue), PayloadID(InPayloadID), FenceValue(InFenceValue), LastCPUSignaledFenceValue(InLastCPUSignaledFenceValue) {}

	double Timestamp;
	uint8 Queue;
	uint32 PayloadID;
	uint64 FenceValue;
	uint64 LastCPUSignaledFenceValue;
};

struct FInterruptWakeUp
{
	UE_NONCOPYABLE(FInterruptWakeUp)
	FInterruptWakeUp(uint32 InThreadID, double InStartTime) : ThreadID(InThreadID), StartTime(InStartTime) {}

	uint32 ThreadID;
	double StartTime;
	double EndTime = 0.0;
	uint8 ExitStatus = 0;

	TArray<FInterruptFenceSignaledEvent, TInlineAllocator<2>> SignalEvents;
};

class RENDERTRACEINSIGHTS_API IRenderTraceProvider : public TraceServices::IProvider
{
public:
	using TEventTimeline = TraceServices::TMonotonicTimeline<FRenderTraceEvent>;

	virtual uint64 GetTimelinesModCount() const = 0;

	// Note: none of the accessor methods which take an item ID do explicit bounds checking. If an invalid ID is passed, the TArray
	// operator[] will assert if checks are enabled, but otherwise garbage will be returned.

	// RHI command list handling.
	virtual int32 GetNumCommandLists() const = 0;
	virtual const FCommandListInstance& GetCommandList(uint32 ID) const = 0;
	virtual int32 GetNumCommandListTimelines() const = 0;
	virtual void ReadCommandListTimelines(TFunctionRef<void(uint32, const TEventTimeline&)> Callback) const = 0;

	// RDG pass handling.
	virtual int32 GetNumRDGPasses() const = 0;
	virtual const FRDGPassInstance& GetRDGPass(uint32 ID) const = 0;
	virtual void ReadRDGTimelines(TFunctionRef<void(uint32, uint32, const TEventTimeline&)> Callback) const = 0;
	virtual const TEventTimeline* GetRenderThreadSubmissionTimeline() const = 0;

	// RHI translate task handling.
	virtual int32 GetNumRHITranslateTasks() const = 0;
	virtual const FRHITranslateTask& GetRHITranslateTask(uint32 ID) const = 0;
	virtual uint32 FindRHITaskByPredicate(uint32 StartAtTaskID, double TimeRange, TFunctionRef<int(const FRHITranslateTask&)> Pred) const = 0;
	virtual void EnumerateRHITranslateTimelines(TFunctionRef<void(uint32, uint32, const TEventTimeline&)> Callback) const = 0;
	virtual const TEventTimeline* GetRHISubmissionTimeline() const = 0;

	// Platform payload handling.
	virtual int32 GetNumPlatformPayloads() const = 0;
	virtual const FPlatformPayload& GetPlatformPayload(uint32 ID) const = 0;

	// Sync point handling.
	virtual int32 GetNumSyncPoints() const = 0;
	virtual const FSyncPoint& GetSyncPoint(uint32 ID) const = 0;

	// Submission queue handling.
	virtual int32 GetNumSubmissionBatches() const = 0;
	virtual const FSubmissionBatch& GetSubmissionBatch(uint32 ID) const = 0;
	virtual const TEventTimeline* GetSubmissionQueueTimeline() const = 0;

	// Interrupt queue handling.
	virtual int32 GetNumInterruptWakeUps() const = 0;
	virtual const FInterruptWakeUp& GetInterruptWakeUp(uint32 ID) const = 0;
	virtual const TEventTimeline* GetInterruptTimeline() const = 0;
};

class RENDERTRACEINSIGHTS_API IEditableRenderTraceProvider : public IRenderTraceProvider, public TraceServices::IEditableProvider
{
public:
	virtual TPair<uint32, FCommandListInstance&> AddCommandList(uint64 AppID, double Timestamp, ECommandListType Type) = 0;
	virtual FCommandListInstance& EditCommandList(uint32 ID) = 0;
	virtual void EnumerateCommandListsForEdit(TFunctionRef<void(uint32, FCommandListInstance&)> Callback) = 0;
	virtual TPair<uint32, TEventTimeline&> AddCommandListTimeline() = 0;
	virtual TEventTimeline& EditCommandListTimeline(uint32 Index) = 0;

	virtual TPair<uint32, FRDGPassInstance&> AddRDGPass(const TCHAR* Name, uint32 ThreadID, double Timestamp, ERDGPassType Type) = 0;
	virtual FRDGPassInstance& EditRDGPass(uint32 ID) = 0;
	virtual TEventTimeline& EditRDGThreadTimeline(uint32 ThreadID) = 0;
	virtual TEventTimeline& EditRenderThreadSubmissionTimeline() = 0;

	virtual TPair<uint32, FRHITranslateTask&> AddRHITranslateTask(uint64 AppID, ERHITranslateTaskType Type, uint32 ThreadID, double Timestamp) = 0;
	virtual FRHITranslateTask& EditRHITranslateTask(uint32 ID) = 0;
	virtual TEventTimeline& EditRHITranslateThreadTimeline(uint32 ThreadID) = 0;
	virtual TEventTimeline& EditRHISubmissionTimeline() = 0;

	virtual TPair<uint32, FPlatformPayload&> AddPlatformPayload(uint64 InAppID, double InStartTime, uint8 InPipeIdx) = 0;
	virtual FPlatformPayload& EditPlatformPayload(uint32 ID) = 0;

	virtual TPair<uint32, FSyncPoint&> AddSyncPoint(uint64 InAppID, ESyncPointType InType) = 0;
	virtual FSyncPoint& EditSyncPoint(uint32 ID) = 0;

	virtual TPair<uint32, FSubmissionBatch&> AddSubmissionBatch(uint32 ThreadID, double Timestamp) = 0;
	virtual FSubmissionBatch& EditSubmissionBatch(uint32 ID) = 0;
	virtual TEventTimeline& EditSubmissionQueueTimeline() = 0;

	virtual TPair<uint32, FInterruptWakeUp&> AddInterruptWakeUp(uint32 ThreadID, double Timestamp) = 0;
	virtual FInterruptWakeUp& EditInterruptWakeUp(uint32 ID) = 0;
	virtual TEventTimeline& EditInterruptTimeline() = 0;
};

RENDERTRACEINSIGHTS_API FName GetRenderTraceProviderName();
RENDERTRACEINSIGHTS_API const IRenderTraceProvider* ReadRenderTraceProvider(const TraceServices::IAnalysisSession& Session);
RENDERTRACEINSIGHTS_API IEditableRenderTraceProvider* EditRenderTraceProvider(TraceServices::IAnalysisSession& Session);

} //namespace RenderTraceInsights
} //namespace UE
